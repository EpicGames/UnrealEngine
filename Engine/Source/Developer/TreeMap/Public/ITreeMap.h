// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/CoreStyle.h"

class FTreeMapAttributeData;
class FTreeMapNodeData;

typedef TSharedRef< class FTreeMapAttributeData > FTreeMapAttributeDataRef;
typedef TSharedPtr< class FTreeMapAttributeData > FTreeMapAttributeDataPtr;


/**
 * Stores data for a specific attribute type on a node
 */
class FTreeMapAttributeData
{

public:

	/** Value for this data */
	FName Value;


public:

	/** Default constructor for FTreeMapAttributeData */
	FTreeMapAttributeData()
		: Value( NAME_None )
	{
	}

	/** Intitialization constructor for FTreeMapAttributeData */
	FTreeMapAttributeData( const FName InitValue )
		: Value( InitValue )
	{
	}
};



typedef TSharedRef< class FTreeMapNodeData > FTreeMapNodeDataRef;
typedef TSharedPtr< class FTreeMapNodeData > FTreeMapNodeDataPtr;

/**
 * Single node in a tree map, which may have any number of child nodes, each with their own children and so on
 */
class FTreeMapNodeData : public TSharedFromThis<FTreeMapNodeData>
{

public:
	/** Logical name, if set this is used for comparison */
	FString LogicalName;

	/** Node name display.  When drawing the tree map, this shows up at the top of the inside of the node's rectangle */
	FString Name;

	/** Node name (line 2).  Left nodes only.  This shows up on the second line of the node, right underneath its name.  Might be drawn in a smaller font by default. */
	FString Name2;

	/** Center info text.  Leaf nodes only.  This draws right in the center of the node, with a larger font by default */
	FString CenterText;

	/** 
	 * Size of this node.  This works differently depending on whether the node is a leaf node (no children) or a container node (has children):
	 *  - Leaf nodes must ALWAYS have a non-zero size.  
	 *	- Container nodes with size of zero will have their size determined automatically by the sum of their child sizes
	 *  - Container nodes with a non-zero size will override their child sizes and always have the explicitly set size.  Sizes of children will still be used to proportionately scale the size of the child nodes within this container.
	 */
	float Size;

	/** Background brush for this node's box (optional) */
	const struct FSlateBrush* BackgroundBrush;

	/** Color for this node.  This will be set automatically unless you use a customization. */
	FLinearColor Color;

	/** Hashtags for this node.  This is just meta-data for the node that may have been loaded from a file.  It is not used by the TreeMap system 
	   directly, but you could harvest these tags and convert them into custom attributes if you wanted to! */
	TArray<FString> HashTags;

	/** Map of attribute name to it's respective bit of data */
	TMap< FName, FTreeMapAttributeDataPtr > Attributes;

	/** Back pointer to parent node, or NULL if no parent exists (root node) */
	FTreeMapNodeData* Parent;

	/** List of child nodes */
	TArray< FTreeMapNodeDataPtr > Children;

public:

	/** Default constructor for FTreeMapNodeData */
	FTreeMapNodeData()
		: Size( 0.0 ),
		  BackgroundBrush( nullptr ),
		  Color( FLinearColor::White ),
		  Parent( NULL )
	{
	}

	bool operator==(const FTreeMapNodeData& Other) const
	{
		if (!LogicalName.IsEmpty() || !Other.LogicalName.IsEmpty())
		{
			return LogicalName == Other.LogicalName;
		}

		return Name == Other.Name && Name2 == Other.Name2 && CenterText == Other.CenterText;
	}

	bool operator!=(const FTreeMapNodeData& Other) const
	{
		return !(*this == Other);
	}

	/** @return Returns true if this is a leaf node */
	bool IsLeafNode() const
	{
		return Children.Num() == 0;
	}

	/** @return Copies this node into another node (not children, though.  The copied node will have no children.) */
	void CopyNodeInto( FTreeMapNodeData& NodeCopy ) const
	{
		NodeCopy.LogicalName = this->LogicalName;
		NodeCopy.Name = this->Name;
		NodeCopy.Name2 = this->Name2;
		NodeCopy.CenterText = this->CenterText;
		NodeCopy.BackgroundBrush = this->BackgroundBrush;
		NodeCopy.Size = this->Size;
		NodeCopy.Color = this->Color;
		NodeCopy.HashTags = this->HashTags;

		for( auto HashIter( this->Attributes.CreateConstIterator() ); HashIter; ++HashIter )
		{
			FTreeMapAttributeDataRef AttributeDataCopy( new FTreeMapAttributeData() );
			AttributeDataCopy->Value = HashIter.Value()->Value;
			NodeCopy.Attributes.Add( HashIter.Key(), AttributeDataCopy );
		}

		NodeCopy.Children.Empty();
		NodeCopy.Parent = this->Parent;
	}

	/** @return Returns a copy of this node; all child nodes are copied, too! */
	FTreeMapNodeDataRef CopyNodeRecursively() const
	{
		FTreeMapNodeDataRef NodeCopy = MakeShareable( new FTreeMapNodeData() );

		this->CopyNodeInto( *NodeCopy );

		for( const auto& ChildNode : this->Children )
		{
			const auto& ChildCopy = ChildNode->CopyNodeRecursively();
			NodeCopy->Children.Add( ChildCopy );
			ChildCopy->Parent = &NodeCopy.Get();
		}

		return NodeCopy;
	}
};


/** Type of tree map */
enum class ETreeMapType
{
	/** Plain tree map */
	Standard,

	/** Squarified tree map */
	Squarified,
};


/**
 * Configuration for a new tree map
 */
struct FTreeMapOptions
{
	/** Width of whole display area */
	float DisplayWidth;

	/** Height of whole display area */
	float DisplayHeight;

	/** Type of tree map */
	ETreeMapType TreeMapType;

	/** Font to use for titles.  This is needed so that we can calculate the amount of padding space needed.  This font will be used for text
	    titles at the root-most level of the tree.  Nested levels get smaller fonts.  This also affects the amount of space reserved for the title area at the top of each node. */
	FSlateFontInfo NameFont;

	/** Font for second line of text, under the title.  Leaf nodes only.  Usually a bit smaller.  Works just like NameFont */
	FSlateFontInfo Name2Font;

	/** Font for any text that's centered inside the middle of the node.  Leaf nodes only.  Usually a bit larger.  Works just like NameFont */
	FSlateFontInfo CenterTextFont;

	/** Number of font sizes to drop with each depth level of the tree */
	int32 FontSizeChangeBasedOnDepth;

	/** Padding around the outside of top-level container nodes in the tree */
	float TopLevelContainerOuterPadding;

	/** Padding around the outside of nested container nodes in the tree */
	float NestedContainerOuterPadding;

	/** Padding around a set of children inside of containers, under the container's title area */
	float ContainerInnerPadding;

	/** Minimize size of a tree node that will be allowed to have a title and padding.  Nodes smaller than this will be a simple rectangle.  Doesn't apply to top level nodes. */
	float MinimumInteractiveNodeSize;

	/** Minimize size of a tree node that will be drawn at all, below this size it is invisible */
	float MinimumVisibleNodeSize;


	/** Default constructor for FTreeMapOptions that initializes good defaults */
	FTreeMapOptions()
		: DisplayWidth( 1.0f ),
		  DisplayHeight( 1.0f ),
		  TreeMapType( ETreeMapType::Standard ),
		  NameFont( FCoreStyle::Get().GetFontStyle( TEXT( "NormalText" ) ) ),
		  Name2Font( FCoreStyle::Get().GetFontStyle( TEXT( "NormalText" ) ) ),
		  CenterTextFont( FCoreStyle::Get().GetFontStyle( TEXT( "NormalText" ) ) ),
		  FontSizeChangeBasedOnDepth( 1 ),
		  TopLevelContainerOuterPadding( 0.0f ),
		  NestedContainerOuterPadding( 0.0f ),
		  ContainerInnerPadding( 0.0f ),
		  MinimumInteractiveNodeSize( 0.0f ),
		  MinimumVisibleNodeSize( 0.0f )
	{
		NameFont.Size = 12;
		Name2Font.Size = 8;
		CenterTextFont.Size = 24;
	}
};



/**
 * Visual ID for a node, generated by the tree map system
 */
struct FTreeMapNodeVisualInfo 
{
	/** Pointer back to the node data this visual was originally created from */
	class FTreeMapNodeData* NodeData;

	/** Position for this node */
	FVector2D Position;

	/** Size for this node */
	FVector2D Size;

	/** Node color */
	FLinearColor Color;

	/** Node fonts */
	FSlateFontInfo NameFont;
	FSlateFontInfo Name2Font;
	FSlateFontInfo CenterTextFont;

	/** True if the node is 'interactive'.  That is, we have space for the node's title and for it to be clicked on. */
	bool bIsInteractive;
};


/**
 * Public tree map interface
 */
class ITreeMap
{

public:

	/**
	 * Gets the visuals for this tree map.  Be careful not to destroy the original tree while still using the visual objects, otherwise
	 * the back pointers to the original tree nodes will be trash!  Visuals are ordered such that nested visuals appear later in the list
	 * than their parents (so they can be drawn back to front, etc.)
	 *
	 * @return List of visuals
	 */
	virtual TArray<FTreeMapNodeVisualInfo> GetVisuals() = 0;

	/**
	 * Static: Creates a tree map object, given tree node source data
	 *
	 * @param	Options			Configuration options for this tree map
	 * @param	RootNodeData	The source data for the root of the tree
	 *
	 * @return	The newly-created ITreeMap instance 
	 */
	static TSharedRef<ITreeMap> CreateTreeMap( const FTreeMapOptions& Options, const FTreeMapNodeDataRef& RootNodeData );

	/**
	 * Parses an OPML XML document and converts the outliner content to tree map node data
	 *
	 * @param	OPMLFilePath		Path to the OPML file on disk
	 * @param	OutErrorMessage		Error message string in the case that something went wrong
	 *
	 * @return	The root tree node for the parsed data, or an invalid pointer if something went wrong
	 */
	static FTreeMapNodeDataPtr ParseOPMLToTreeMapData( const FString& OPMLFilePath, FString& OutErrorMessage );

	/** Virtual destructor */
	virtual ~ITreeMap() = default;
};



