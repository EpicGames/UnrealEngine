// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "ITreeMap.h"

class FTreeMapAttribute;
class FTreeMapAttributeValue;

typedef TSharedPtr< class FTreeMapAttributeValue > FTreeMapAttributeValuePtr;
typedef TSharedRef< class FTreeMapAttributeValue > FTreeMapAttributeValueRef;

/**
 * Describes a value for an attribute of a tree node
 */
class FTreeMapAttributeValue
{

public:

	/** Name of this value */
	FName Name;

	/** The node size that this value maps to */
	float NodeSize;

	/** The node color that this value maps to */
	FLinearColor NodeColor;


public:

	/** Default constructor */
	FTreeMapAttributeValue()
		: Name(),
		  NodeSize( 1.0f ),
		  NodeColor( FLinearColor::White )
	{
	}


	/** Virtual destructor */
	virtual ~FTreeMapAttributeValue() = default;
};



typedef TSharedPtr< class FTreeMapAttribute > FTreeMapAttributePtr;
typedef TSharedRef< class FTreeMapAttribute > FTreeMapAttributeRef;

/**
 * Describes a customized attribute of a tree node
 */
class FTreeMapAttribute
{

public:

	/** Name of this attribute */
	FName Name;

	/** Maps a value name to the data that describes that value */
	TMap< FName, FTreeMapAttributeValuePtr > Values;

	/** Default value to use when none is specified on a node */
	FTreeMapAttributeValuePtr DefaultValue;


public:

	/** Default constructor */
	FTreeMapAttribute()
		: Name(),
		  Values(),
		  DefaultValue()
	{
	}

	/** Virtual destructor */
	virtual ~FTreeMapAttribute() = default;

};



/**
 * Implement ITreeMapCustomization and pass it along when creating your tree map to allow for custom attributes and formatting features!
 */
class ITreeMapCustomization
{

public:

	/** @return Returns the name of this customization */
	virtual FName GetName() const = 0;

	/** @return Returns all of the possible attributes in this customization.  Each attribute defines a set of possible values for that attribute. */
	virtual const TMap< FName, FTreeMapAttributePtr >& GetAttributes() const = 0;

	/** @return Returns the default attribute type to size the nodes based on */
	virtual FTreeMapAttributePtr GetDefaultSizeByAttribute() const { return NULL; }

	/** @return Returns the default attribute type to color the nodes based on */
	virtual FTreeMapAttributePtr GetDefaultColorByAttribute() const { return NULL; }

	/**
	 * Optional override that can be called to convert hash tags on tree nodes into proper attribute values.  This is useful if you've loaded
	 * tree node data from a file and want to take the raw hash tag strings and make values for those.
	 *
	 * @param	Node	The root node to process
	 */
	virtual void ProcessHashTagsRecursively( const FTreeMapNodeDataRef& Node ) const {};

	/** Virtual destructor */
	virtual ~ITreeMapCustomization() = default;
};



