// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Animation/CurveSequence.h"
#include "Widgets/SLeafWidget.h"
#include "ITreeMap.h"
#include "ITreeMapCustomization.h"
#include "TreeMapStyle.h"

#define UE_API TREEMAP_API

class FPaintArgs;
class FSlateWindowElementList;
class FWidgetPath;

/**
 * Graphical tree map widget with interactive controls
 */
class STreeMap : public SLeafWidget
{

public:

	/** Delegate used when clicking or interacting with a specific node */
	DECLARE_DELEGATE_TwoParams( FOnTreeMapNodeInteracted, FTreeMapNodeData&, const FPointerEvent& );

	SLATE_BEGIN_ARGS( STreeMap )
		: _AllowEditing( false )
		, _BackgroundImage( FTreeMapStyle::Get().GetBrush( "TreeMap.Background" ) )
		, _NodeBackground( FTreeMapStyle::Get().GetBrush( "TreeMap.NodeBackground" ) )
		, _HoveredNodeBackground( FTreeMapStyle::Get().GetBrush( "TreeMap.HoveredNodeBackground" ) )
		, _NameFont()
		, _Name2Font()
		, _CenterTextFont()
		, _BorderPadding( FTreeMapStyle::Get().GetVector( "TreeMap.BorderPadding" ) )
		, _MinimumInteractiveTreeNodeSize( 64 * 64 )
		, _MinimumVisibleTreeNodeSize( 6 * 6 )
		, _NavigationTransitionTime( 0.25f )
		, _TopLevelContainerOuterPadding( 4.0f )
		, _NestedContainerOuterPadding( 0.0f )
		, _ContainerInnerPadding( 4.0f )
		, _ChildContainerTextPadding( 2.0f )
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}

		/** Sets whether the user can edit the tree map interactively by dragging nodes around and typing new node labels */
		SLATE_ATTRIBUTE( bool, AllowEditing )

		/** Background image to use for the tree map canvas area */
		SLATE_ATTRIBUTE( const FSlateBrush*, BackgroundImage )

		/** Background to use for each tree node */
		SLATE_ATTRIBUTE( const FSlateBrush*, NodeBackground )
			
		/** Background to use for nodes that the mouse is hovering over */
		SLATE_ATTRIBUTE( const FSlateBrush*, HoveredNodeBackground )

		/** Sets the font used to draw the text.  Note the height of the font may change automatically based on the tree node size */
		SLATE_ATTRIBUTE( FSlateFontInfo, NameFont )
			
		/** Font for second line of text, under the title.  Leaf nodes only.  Usually a bit smaller.  Works just like NameFont */
		SLATE_ATTRIBUTE( FSlateFontInfo, Name2Font );

		/** Font for any text that's centered inside the middle of the node.  Leaf nodes only.  Usually a bit larger.  Works just like NameFont */
		SLATE_ATTRIBUTE( FSlateFontInfo, CenterTextFont );

		/** Border Padding around fill bar */
		SLATE_ATTRIBUTE( FVector2D, BorderPadding )

		/** Minimum size of node that can be interacted with, if small you need to drill down into parent first */
		SLATE_ARGUMENT( int32, MinimumInteractiveTreeNodeSize );

		/** Minimum size in pixels of a tree node that we should bother including in the UI.  Below this size, you'll need to drill down to see the node. */
		SLATE_ARGUMENT( int32, MinimumVisibleTreeNodeSize );

		/** How many seconds to animate the visual transition when the user navigates to a new tree node, or after a modification of the tree takes place */
		SLATE_ARGUMENT( float, NavigationTransitionTime );

		/** How many pixels of padding around the outside of the root-level tree node box.  Adding padding makes the tree look better, but you lose some sizing accuracy */
		SLATE_ARGUMENT( float, TopLevelContainerOuterPadding );

		/** How many pixels of padding around the outside of a non-root tree node's box.  Adding padding makes the tree look better, but you lose some sizing accuracy */
		SLATE_ARGUMENT( float, NestedContainerOuterPadding );

		/** How many pixels of spacing between the container and its child containers.  Adding padding makes the tree look better, but you lose some sizing accuracy */
		SLATE_ARGUMENT( float, ContainerInnerPadding );

		/** How many pixels to pad text that's drawn inside of a container (not the top-level container, though) */
		SLATE_ARGUMENT( float, ChildContainerTextPadding );

		/** Optional delegate that fires when the node is double-clicked in the tree.  If you don't override this, the tree will use its
			default handling of double-click, which will re-root the tree on the node under the cursor */
		SLATE_EVENT( FOnTreeMapNodeInteracted, OnTreeMapNodeDoubleClicked )

		/** Optional delegate that fires when the node is right-clicked in the tree.  If you don't override this nothing will happen unless a customization is specified */
		SLATE_EVENT( FOnTreeMapNodeInteracted, OnTreeMapNodeRightClicked )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param	InArgs				A declaration from which to construct the widget
	 * @param	InTreeMapNodeData	The node data we'll be visualizing and editing (root node)
	 * @param	InCustomization		An optional 'customization' of the tree that allows for filtering and displaying the tree map data according to various attributes
	 */
	UE_API void Construct( const FArguments& InArgs, const TSharedRef<class FTreeMapNodeData>& InTreeMapNodeData, const TSharedPtr< ITreeMapCustomization >& InCustomization );

	/** SWidget overrides */
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	UE_API virtual FReply OnMouseMove( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	UE_API virtual void OnMouseLeave( const FPointerEvent& InMouseEvent ) override;
	UE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	UE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyboardEvent ) override;

	/**
	 * Sets a new "active root" for the tree.  This is used to "drill down" to child tree nodes or "climb up" back to the root.
	 * This also happens automatically when the user presses the mouse wheel to zoom in and out over nodes.
	 * 
	 * @param	NewRoot						The tree's new root.  This can be any node within the tree
	 * @param	bShouldPlayTransition		If enabled, an animation will play
	 */
	UE_API void SetTreeRoot( const FTreeMapNodeDataRef& NewRoot, const bool bShouldPlayTransition );

	/** Returns true if it is possible to zoom out */
	UE_API bool CanZoomOut() const;

	/** Zooms out, setting root to it's parent if possible, returns false if not */
	UE_API bool ZoomOut();

	/** Returns the current tree root, which is the largest node visible */
	UE_API FTreeMapNodeDataPtr GetTreeRoot() const;

	/**
	 * Refreshes the tree map from its source data.  The current active root will remain as the base of the tree.  Call this
	 * after you've changed data in the tree.
	 * 
	 * @param	bShouldPlayTransition		If enabled, an animation will play
	 */
	UE_API void RebuildTreeMap( const bool bShouldPlayTransition );
		

protected:

	/** Finds the node visual that's under the cursor */
	UE_API struct FTreeMapNodeVisualInfo* FindNodeVisualUnderCursor( const FGeometry& MyGeometry, const FVector2D& ScreenSpaceCursorPosition );

	/** Blends between the current visual state and the state before the last navigation transition, by the specified amount */
	UE_API void MakeBlendedNodeVisual( const int32 VisualIndex, const float NavigationAlpha, FTreeMapNodeVisualInfo& OutVisual ) const;

	/** @return True if we're playing a transition after a navigation action (zoom!) */
	UE_API bool IsNavigationTransitionActive() const;

	/** Grabs a snapshot of the current tree so we can restore it if the user performs an Undo action */
	UE_API void TakeUndoSnapshot();

	/** Undoes the last action */
	UE_API void Undo();

	/** Redoes the last action */
	UE_API void Redo();

	/** Reparents DroppedNode to NewParentNode (undoable!) */
	UE_API void ReparentNode( const FTreeMapNodeDataRef DroppedNode, const FTreeMapNodeDataRef NewParentNode );

	/** Deletes the node under the mouse cursor, if any (undoable!) */
	UE_API FReply DeleteHoveredNode();

	/** Insert a new node as a child of the node under the cursor */
	UE_API FReply InsertNewNodeAsChildOfHoveredNode( const FGeometry& MyGeometry );

	/** Searches for the specified node in an identical copy of the node tree. */
	UE_API FTreeMapNodeDataPtr FindNodeInCopiedTree( const FTreeMapNodeDataRef& NodeToFind, const FTreeMapNodeDataRef& OriginalNode, const FTreeMapNodeDataRef& CopiedRootNode ) const;

	/** Pops up a box to allow the user to start renaming a node's title (undoable!) */
	UE_API void StartRenamingNode( const FGeometry& MyGeometry, const FTreeMapNodeDataRef& NodeData, const FVector2D& RelativePosition, const bool bIsNewNode );

	/** Called when the user commits a rename change */
	UE_API void RenamingNode_OnTextCommitted( const FText& NewText, ETextCommit::Type, TSharedRef<FTreeMapNodeData> NodeToRename );

	/** Stops renaming a node, committing whatever text was entered */
	UE_API void StopRenamingNode();

	/** Called to apply the current 'size based on' and 'color based on' settings to the tree */
	UE_API void ApplyVisualizationToNodes( const FTreeMapNodeDataRef& Node );

	/** Recursively applies the active visualization to all nodes, such as size by attribute, or color by attribute */
	UE_API void ApplyVisualizationToNodesRecursively( const FTreeMapNodeDataRef& Node, const FLinearColor& DefaultColor, const int32 TreeDepth );

	/** Displays a context menu at the specified location with options for configuring the tree display */
	UE_API void ShowOptionsMenuAt( const FPointerEvent& InMouseEvent );

private:
	UE_API void ShowOptionsMenuAtInternal(const FVector2D& ScreenSpacePosition, const FWidgetPath& WidgetPath);

private:

	/**
	 * TreeMap data
	 */

	/** The tree map that we're presenting using this widget */
	TSharedPtr<class ITreeMap> TreeMap;

	/** Customization for this tree map */
	TSharedPtr<class ITreeMapCustomization> Customization;

	/** The previous tree map for the last view we were looking at */
	TSharedPtr<class ITreeMap> LastTreeMap;

	/** The source data for the tree map */
	FTreeMapNodeDataPtr TreeMapNodeData;

	/** Current root of the tree that we're visualizing.  The user can re-root the tree by double-clicking on tree map nodes */
	FTreeMapNodeDataPtr ActiveRootTreeMapNode;

	/** Which attribute to visualize the size of the nodes based on, or null for default sizes */
	FTreeMapAttributePtr SizeNodesByAttribute;

	/** Which attribute to visualize the color of the nodes based on, or null for automatic colors */
	FTreeMapAttributePtr ColorNodesByAttribute;


	/**
	 * Widget layout
	 */

	/** Background image to use for the tree map canvas area */
	TAttribute< const FSlateBrush* > BackgroundImage;

	/** Background to use for each tree node */
	TAttribute< const FSlateBrush* > NodeBackground;

	/** Background to use for nodes that the mouse is hovering over */
	TAttribute< const FSlateBrush* > HoveredNodeBackground;

	/** Border Padding */
	TAttribute<FVector2D> BorderPadding;

	/** The font name used to draw the text */
	TAttribute< FSlateFontInfo > NameFont;

	/** Font for second line of text, under the title.  Leaf nodes only.  Usually a bit smaller.  Works just like NameFont */
	TAttribute< FSlateFontInfo > Name2Font;

	/** Font for any text that's centered inside the middle of the node.  Leaf nodes only.  Usually a bit larger.  Works just like NameFont */
	TAttribute< FSlateFontInfo > CenterTextFont;


	/**
	 * Navigation
	 */

	/** The visual that the mouse was over last */
	FTreeMapNodeVisualInfo* MouseOverVisual;

	/** Optional delegate that fires when the node is double-clicked in the tree.  If you don't override this, the tree will use its
		default handling of double-click, which will re-root the tree on the node under the cursor */
	FOnTreeMapNodeInteracted OnTreeMapNodeDoubleClicked;

	/** Right click event */
	FOnTreeMapNodeInteracted OnTreeMapNodeRightClicked;

	/**
	 * TreeMap visuals
	 */

	/** Minimum size of node that can be interacted with, if small you need to drill down into parent first */
	int32 MinimumInteractiveTreeNodeSize;

	/** Minimum size in pixels of a tree node that we should bother including in the UI.  Below this size, you'll need to drill down to see the node. */
	int32 MinimumVisibleTreeNodeSize;

	/** Size of our geometry the last time we rebuilt the tree map */
	FVector2D TreeMapSize;

	/** Cached set of visuals for the tree map the last time it was generated */
	TArray<struct FTreeMapNodeVisualInfo> CachedNodeVisuals;

	/** Cached node visuals for last tree map */
	TArray<struct FTreeMapNodeVisualInfo> LastCachedNodeVisuals;

	/** Maps node visual indices to their last node visual indices.  This is used for navigation transitions */
	TMap<int32, int32> NodeVisualIndicesToLastIndices;

	/** List of node visual indices which no longer map to a node in the current layout. This is used for navigation transitions */
	TArray<int32> OrphanedLastIndices;

	/** Time that tree view was changed last after navigation */
	FCurveSequence NavigateAnimationCurve;

	/** How many pixels of padding around the outside of the root-level tree node box.  Adding padding makes the tree look better, but you lose some sizing accuracy */
	float TopLevelContainerOuterPadding;

	/** How many pixels of padding around the outside of a non-root tree node's box.  Adding padding makes the tree look better, but you lose some sizing accuracy */
	float NestedContainerOuterPadding;

	/** How many pixels of spacing between the container and its child containers.  Adding padding makes the tree look better, but you lose some sizing accuracy */
	float ContainerInnerPadding;

	/** How many pixels to pad text that's drawn inside of a container (not the top-level container, though) */
	float ChildContainerTextPadding;


	/**
	 * Live editing
	 */

	/** True if we should allow real-time editing right now.  The user can still navigate between nodes even if editing is disbled, so its not exactly "read only" */
	TAttribute<bool> AllowEditing;

	/** Node that we're currently dragging around.  This points to an entry in the CachedNodeVisuals array! */
	FTreeMapNodeVisualInfo* DraggingVisual;

	/** Distance we've LMB+dragged the cursor.  Used to determine if we're ready to start dragging and dropping */
	float DragVisualDistance;

	/** Position of the mouse cursor relative to the widget, where the user picked up a dragged visual */
	FVector2D RelativeDragStartMouseCursorPos;

	/** Position of the mouse cursor relative to the widget, the last time it moved.  Used for drag and drop. */
	FVector2D RelativeMouseCursorPos;

	/** Undo history buffer. */
	TArray< TSharedPtr< FTreeMapNodeData > > UndoStates;

	/** Current undo level, or INDEX_NONE if we haven't undone anything */
	int32 CurrentUndoLevel;

	/** Weak pointer to the widget used to rename a node inline.  We might need to close the window in some cases. */
	TWeakPtr<SWidget> RenamingNodeWidget;

	/** Weak pointer to the node that we're currently renaming, if any */
	TWeakPtr<FTreeMapNodeData> RenamingNodeData;

	/** True if the node we're renaming is a newly-created node that should be added to the tree after we finish naming it */
	bool bIsNamingNewNode;

	/** Node that we're currently drawing a highlight pulse effect for.  This points to an entry in the CachedNodesVisuals array */
	TWeakPtr<FTreeMapNodeData> HighlightPulseNode;

	/** Time that we started playing a highlight pulse effect for the HighlightPulseNode */
	double HighlightPulseStartTime;

};

#undef UE_API
