// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

#define UE_API TREEMAP_API

/**
 * Style data for STreeMap
 */
class FTreeMapStyle
{
public:
	static UE_API void Initialize();
	static UE_API void Shutdown();
	static UE_API const ISlateStyle& Get();
	static UE_API FName GetStyleSetName();

private:

	static UE_API TSharedRef< class FSlateStyleSet > Create();

private:

	static UE_API TSharedPtr< class FSlateStyleSet > StyleInstance;
};

#undef UE_API
