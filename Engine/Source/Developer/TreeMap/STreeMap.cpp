// Copyright Epic Games, Inc. All Rights Reserved.

#include "STreeMap.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"


#define LOCTEXT_NAMESPACE "STreeMap"


namespace STreeMapDefs
{
	/** Minimum pixels the user must drag the cursor before a drag+drop starts on a visual */
	const float MinCursorDistanceForDraggingVisual = 3.0f;
}



void STreeMap::Construct( const FArguments& InArgs, const TSharedRef<FTreeMapNodeData>& InTreeMapNodeData, const TSharedPtr< ITreeMapCustomization >& InCustomization )
{
	CurrentUndoLevel = INDEX_NONE;
	Customization = InCustomization;

	TreeMapNodeData = InTreeMapNodeData;

	AllowEditing = InArgs._AllowEditing;

	BackgroundImage = InArgs._BackgroundImage;
	NodeBackground = InArgs._NodeBackground;
	HoveredNodeBackground = InArgs._HoveredNodeBackground;
	BorderPadding = InArgs._BorderPadding;
	RelativeDragStartMouseCursorPos = FVector2D::ZeroVector;
	RelativeMouseCursorPos = FVector2D::ZeroVector;

	{
		FTreeMapOptions TreeMapOptions;
		NameFont = TreeMapOptions.NameFont;
		Name2Font = TreeMapOptions.Name2Font;
		CenterTextFont = TreeMapOptions.CenterTextFont;

		if( InArgs._NameFont.IsSet() )
		{
			NameFont = InArgs._NameFont;
		}
		if( InArgs._Name2Font.IsSet() )
		{
			Name2Font = InArgs._Name2Font;
		}
		if( InArgs._CenterTextFont.IsSet() )
		{
			CenterTextFont = InArgs._CenterTextFont;
		}
	}

	MinimumInteractiveTreeNodeSize = InArgs._MinimumInteractiveTreeNodeSize;
	MinimumVisibleTreeNodeSize = InArgs._MinimumVisibleTreeNodeSize;
	TopLevelContainerOuterPadding = InArgs._TopLevelContainerOuterPadding;
	NestedContainerOuterPadding = InArgs._NestedContainerOuterPadding;
	ContainerInnerPadding = InArgs._ContainerInnerPadding;
	ChildContainerTextPadding = InArgs._ChildContainerTextPadding;

	TreeMapSize = FVector2D( 0, 0 );
	NavigateAnimationCurve.AddCurve( 0.0f, InArgs._NavigationTransitionTime, ECurveEaseFunction::CubicOut );
	MouseOverVisual = nullptr;
	DraggingVisual = nullptr;
	DragVisualDistance = 0.0f;

	bIsNamingNewNode = false;

	HighlightPulseStartTime = -99999.0;

	OnTreeMapNodeDoubleClicked = InArgs._OnTreeMapNodeDoubleClicked;
	OnTreeMapNodeRightClicked = InArgs._OnTreeMapNodeRightClicked;

	if( Customization.IsValid() )
	{
		SizeNodesByAttribute = Customization->GetDefaultSizeByAttribute();
		ColorNodesByAttribute = Customization->GetDefaultColorByAttribute();
	}

	const bool bShouldPlayTransition = false;
	SetTreeRoot( TreeMapNodeData.ToSharedRef(), bShouldPlayTransition );
}


void STreeMap::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SLeafWidget::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	if( AllottedGeometry.Size != TreeMapSize || !TreeMap.IsValid() )
	{
		// Stop renaming a node if we were doing that
		StopRenamingNode();

		// Make a tree map!
		FTreeMapOptions TreeMapOptions;
		TreeMapOptions.TreeMapType = ETreeMapType::Squarified;
		TreeMapOptions.DisplayWidth = AllottedGeometry.GetLocalSize().X;
		TreeMapOptions.DisplayHeight = AllottedGeometry.GetLocalSize().Y;
		TreeMapOptions.TopLevelContainerOuterPadding = TopLevelContainerOuterPadding;
		TreeMapOptions.NestedContainerOuterPadding = NestedContainerOuterPadding;
		TreeMapOptions.ContainerInnerPadding = ContainerInnerPadding;
		TreeMapOptions.NameFont = NameFont.Get();
		TreeMapOptions.Name2Font = Name2Font.Get();
		TreeMapOptions.CenterTextFont = CenterTextFont.Get();
		TreeMapOptions.FontSizeChangeBasedOnDepth = 2;		// @todo treemap custom: Expose as a customization option to STreeMap
		TreeMapOptions.MinimumInteractiveNodeSize = MinimumInteractiveTreeNodeSize;
		TreeMapOptions.MinimumVisibleNodeSize = MinimumVisibleTreeNodeSize;
		TreeMap = ITreeMap::CreateTreeMap( TreeMapOptions, ActiveRootTreeMapNode.ToSharedRef() );

		TreeMapSize = AllottedGeometry.Size;

		CachedNodeVisuals = TreeMap->GetVisuals();
		MouseOverVisual = nullptr;
		DraggingVisual = nullptr;

		// Map the new visuals back to the old ones!
		NodeVisualIndicesToLastIndices.Reset();
		TSet<int32> ValidLastIndices;
		for( auto VisualIndex = 0; VisualIndex < CachedNodeVisuals.Num(); ++VisualIndex )
		{
			const auto& Visual = CachedNodeVisuals[ VisualIndex ];
			for( auto LastVisualIndex = 0; LastVisualIndex < LastCachedNodeVisuals.Num(); ++LastVisualIndex )
			{
				const auto& LastVisual = LastCachedNodeVisuals[ LastVisualIndex ];
				if( LastVisual.NodeData == Visual.NodeData )
				{
					NodeVisualIndicesToLastIndices.Add( VisualIndex, LastVisualIndex );
					ValidLastIndices.Add( LastVisualIndex );
					break;
				}
			}
		}

		// Find all of the orphans
		OrphanedLastIndices.Reset();
		for( auto LastVisualIndex = 0; LastVisualIndex < LastCachedNodeVisuals.Num(); ++LastVisualIndex )
		{
			if( !ValidLastIndices.Contains( LastVisualIndex ) )
			{
				OrphanedLastIndices.Add( LastVisualIndex );
			}
		}
	}
}


void STreeMap::MakeBlendedNodeVisual( const int32 VisualIndex, const float NavigationAlpha, FTreeMapNodeVisualInfo& OutVisual ) const
{
	OutVisual = CachedNodeVisuals[ VisualIndex ];	// NOTE: Copying visual

	// Do we need to interp?
	if( NavigationAlpha < 1.0f )
	{
		// Did the visual exist before we navigated?
		const int32* LastVisualIndexPtr = NodeVisualIndicesToLastIndices.Find( VisualIndex );
		if( LastVisualIndexPtr != nullptr )
		{
			// It did exist!
			const auto LastVisualIndex = *LastVisualIndexPtr;
			const auto& LastVisual = LastCachedNodeVisuals[ LastVisualIndex ];

			// Blend before "before" and "now"
			OutVisual.Position = FMath::Lerp( LastVisual.Position, OutVisual.Position, NavigationAlpha );
			OutVisual.Size = FMath::Lerp( LastVisual.Size, OutVisual.Size, NavigationAlpha );

			// Do an HSV color lerp; it just looks more sensible!
			OutVisual.Color = FLinearColor::LerpUsingHSV( LastVisual.Color, OutVisual.Color, NavigationAlpha );

			// The blended visual is considered interactive only if both the new version and the old version were interactive
			OutVisual.bIsInteractive = LastVisual.bIsInteractive && OutVisual.bIsInteractive;
		}
		else
		{
			// Didn't exist before.  Fade in from nothing!
			OutVisual.Color.A *= NavigationAlpha;
		}
	}
}



int32 STreeMap::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const auto DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	// Draw background border layer
	{
		const FSlateBrush* ThisBackgroundImage = BackgroundImage.Get();
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			ThisBackgroundImage,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * ThisBackgroundImage->TintColor.GetColor( InWidgetStyle )
			);
	}

	float NavigationAlpha = NavigateAnimationCurve.GetLerp();


	// Draw tree map
	if( TreeMap.IsValid() )
	{
		// Figure out all the nodes we need to draw in a big ordered list.  This can sometimes include nodes from the previous tree map we were
		// looking at before we "navigated", as those nodes will be briefly visible during the animated transition
		static TArray< const FTreeMapNodeVisualInfo* > NodeVisualsToDraw;
		NodeVisualsToDraw.Reset();

		// Draw the orphan's first
		// @todo treemap visuals: During transitions, nodes look "on top" should be drawn last.  Right now they will underlap and overlap adjacents at the same time.  Looks weird.
		for( auto OrphanedVisualIndex : OrphanedLastIndices )
		{
			NodeVisualsToDraw.Add( &LastCachedNodeVisuals[ OrphanedVisualIndex ] );
		}
		const int32 FirstNewVisualIndex = NodeVisualsToDraw.Num();
		for( const auto& Visual : CachedNodeVisuals )
		{
			NodeVisualsToDraw.Add( &Visual );
		}

		const FTreeMapNodeData* RenamingNodeDataPtr = RenamingNodeData.Pin().Get();

		// Draw background boxes for all visuals
		{
			++LayerId;

			const FSlateBrush* ThisHoveredNodeBackground = HoveredNodeBackground.Get();
			const FSlateBrush* ThisNodeBackground = NodeBackground.Get();

			for( auto DrawVisualIndex = 0; DrawVisualIndex < NodeVisualsToDraw.Num(); ++DrawVisualIndex )
			{
				FTreeMapNodeVisualInfo BlendedVisual;
				if( DrawVisualIndex >= FirstNewVisualIndex )
				{
					MakeBlendedNodeVisual( DrawVisualIndex - FirstNewVisualIndex, NavigationAlpha, BlendedVisual );
				}
				else
				{
					// This is an orphan
					BlendedVisual = *NodeVisualsToDraw[ DrawVisualIndex ];
					BlendedVisual.Color.A *= 1.0f - NavigationAlpha;	// Fade orphans out when navigating
				}

				// Don't draw if completely faded out
				if( BlendedVisual.Color.A > KINDA_SMALL_NUMBER )
				{
					const bool bIsMouseOverNode = MouseOverVisual != nullptr && MouseOverVisual->NodeData == BlendedVisual.NodeData;

					// Draw the visual's background box
					const FVector2D VisualPosition = BlendedVisual.Position;
					const FSlateRect VisualClippingRect = TransformRect( AllottedGeometry.GetAccumulatedLayoutTransform(), FSlateRect( VisualPosition, VisualPosition + BlendedVisual.Size ) );
					const auto VisualPaintGeometry = AllottedGeometry.ToPaintGeometry( BlendedVisual.Size, FSlateLayoutTransform(VisualPosition) );
					auto DrawColor = InWidgetStyle.GetColorAndOpacityTint() * ThisNodeBackground->TintColor.GetColor( InWidgetStyle ) * BlendedVisual.Color;

					OutDrawElements.PushClip(FSlateClippingZone(VisualClippingRect));
					
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						VisualPaintGeometry,
						bIsMouseOverNode ? ThisHoveredNodeBackground : ThisNodeBackground,
						DrawEffects,
						DrawColor
						);

					if( BlendedVisual.NodeData->BackgroundBrush != nullptr )
					{
						// Preserve aspect ratio, but stretch to fill the whole rectangle, even if we have to crop the smaller edge
						const float LargestSize = FMath::Max( BlendedVisual.Size.X, BlendedVisual.Size.Y );
						const FVector2D BackgroundSize( LargestSize, LargestSize );
						FVector2D BackgroundPosition = VisualPosition;
						if( BlendedVisual.Size.X > BlendedVisual.Size.Y )	// Is our box wider than tall?
						{
							const float SizeDifference = LargestSize - BlendedVisual.Size.Y;
							BackgroundPosition.Y -= SizeDifference * 0.5f;
						}
						else
						{
							const float SizeDifference = LargestSize - BlendedVisual.Size.X;
							BackgroundPosition.X -= SizeDifference * 0.5f;
						}
						const auto BackgroundPaintGeometry = AllottedGeometry.ToPaintGeometry( BackgroundSize, FSlateLayoutTransform(BackgroundPosition) );

						const FSlateRect BackgroundClippingRect = VisualClippingRect.InsetBy( FMargin( 1 ) );

						OutDrawElements.PushClip(FSlateClippingZone(BackgroundClippingRect));

						// Draw the background brush
						FSlateDrawElement::MakeBox(
							OutDrawElements,
							LayerId,
							BackgroundPaintGeometry,
							BlendedVisual.NodeData->BackgroundBrush,
							DrawEffects,
							DrawColor );

						OutDrawElements.PopClip();
					}

					const bool bIsHighlightPulseNode = HighlightPulseNode.IsValid() && HighlightPulseNode.Pin().Get() == BlendedVisual.NodeData;
					if( bIsHighlightPulseNode )
					{
						const float HighlightPulseAnimDuration = 1.5f;	// @todo treemap: Probably should be customizable in the widget's settings
						const float HighlightPulseAnimProgress = ( FSlateApplication::Get().GetCurrentTime() - HighlightPulseStartTime ) / HighlightPulseAnimDuration;
						if( HighlightPulseAnimProgress >= 0.0f && HighlightPulseAnimProgress <= 1.0f )
						{
							DrawColor = FLinearColor( 1.0f, 1.0f, 1.0f, FMath::MakePulsatingValue( HighlightPulseAnimProgress, 6.0f, 0.5f ) );

							FSlateDrawElement::MakeBox(
								OutDrawElements,
								LayerId,
								VisualPaintGeometry,
								ThisHoveredNodeBackground,
								DrawEffects,
								DrawColor );
						}
					}

					OutDrawElements.PopClip();
				}
			}
		}


		// Draw text layers.  We draw it twice for all visuals.  Once for a drop shadow, and then again for the foreground text.
		const TSharedRef< FSlateFontMeasure >& FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		for( auto TextLayerIndex = 0; TextLayerIndex < 2; ++TextLayerIndex )
		{
			++LayerId;
			
			// @todo treemap: Want ellipses for truncated text
			// @todo treemap: Squash text based on box size rather than strictly depth?

			const float ShadowOpacity = 0.5f;
			const auto ShadowOffset = ( TextLayerIndex == 0 ) ? FVector2D( -1.0f, 1.0f ) : FVector2D::ZeroVector;
			const auto TextColorScale = ( TextLayerIndex == 0 ) ? FLinearColor( 0.0f, 0.0f, 0.0f, ShadowOpacity ) : FLinearColor::White;

			for( auto DrawVisualIndex = 0; DrawVisualIndex < NodeVisualsToDraw.Num(); ++DrawVisualIndex )
			{
				FTreeMapNodeVisualInfo BlendedVisual;
				if( DrawVisualIndex >= FirstNewVisualIndex )
				{
					MakeBlendedNodeVisual( DrawVisualIndex - FirstNewVisualIndex, NavigationAlpha, BlendedVisual );
				}
				else
				{
					// This is an orphan
					BlendedVisual = *NodeVisualsToDraw[ DrawVisualIndex ];
					BlendedVisual.Color.A *= 1.0f - NavigationAlpha;	// Fade orphans out when navigating
				}

				// Don't draw text if completely faded out, or if not interactive
				if( BlendedVisual.Color.A > KINDA_SMALL_NUMBER && BlendedVisual.bIsInteractive )
				{
					// Don't draw a title for a node that is actively being renamed -- the text box is already visible, after all
					if( BlendedVisual.NodeData != RenamingNodeDataPtr )
					{
						const FVector2D VisualPosition = BlendedVisual.Position;

						// Allow text to fade out with its parent box
						auto VisualTextColor = TextColorScale;
						VisualTextColor.A *= BlendedVisual.Color.A;

						const bool bIsMouseOverNode = MouseOverVisual != nullptr && MouseOverVisual->NodeData == BlendedVisual.NodeData;

						// If the visual is crushed down pretty small in screen space, we don't want to bother even try drawing text
						const FVector2D ScreenSpaceVisualSize( AllottedGeometry.Scale * BlendedVisual.Size );
						if( ScreenSpaceVisualSize.X > 20 )
						{
							FGeometry VisualGeometry = AllottedGeometry.MakeChild(BlendedVisual.Size, FSlateLayoutTransform(VisualPosition));
							const FPaintGeometry VisualPaintGeometry = VisualGeometry.ToPaintGeometry();

							
							// Clip the text to the visual's rectangle, with some extra inner padding to avoid overlapping the visual's border
							FSlateRect TextClippingRect = VisualGeometry.GetLayoutBoundingRect();
							TextClippingRect = TextClippingRect.InsetBy( FMargin( ChildContainerTextPadding, 0, ChildContainerTextPadding, 0 ) );
							if( TextClippingRect.IsValid() )
							{
								OutDrawElements.PushClip(FSlateClippingZone(TextClippingRect));

								// Name (first line)
								float NameTextHeight = 0.0f;
								if( !BlendedVisual.NodeData->Name.IsEmpty() )
								{
									const FVector2D TextSize = FontMeasureService->Measure( BlendedVisual.NodeData->Name, BlendedVisual.NameFont );
									NameTextHeight = TextSize.Y;
									const auto TextX = VisualPosition.X + FMath::Max( ChildContainerTextPadding, (float)BlendedVisual.Size.X * 0.5f - TextSize.X * 0.5f );	// Clamp to left edge if cropped, so the user can at least read the beginning
									const auto TextY = VisualPosition.Y + ChildContainerTextPadding;

									FSlateDrawElement::MakeText(
										OutDrawElements,
										LayerId,
										AllottedGeometry.ToOffsetPaintGeometry( ShadowOffset + FVector2D( TextX, TextY ) ),
										BlendedVisual.NodeData->Name,
										BlendedVisual.NameFont,
										DrawEffects,
										InWidgetStyle.GetColorAndOpacityTint() * VisualTextColor );
								}

								// Name (second line)
								float Name2TextHeight = 0.0f;
								if( !BlendedVisual.NodeData->Name2.IsEmpty() && BlendedVisual.NodeData->IsLeafNode() )
								{
									const FVector2D TextSize = FontMeasureService->Measure( BlendedVisual.NodeData->Name2, BlendedVisual.Name2Font );
									Name2TextHeight = TextSize.Y;
									const auto TextX = VisualPosition.X + FMath::Max( ChildContainerTextPadding, (float)BlendedVisual.Size.X * 0.5f - TextSize.X * 0.5f );	// Clamp to left edge if cropped, so the user can at least read the beginning
									const auto TextY = VisualPosition.Y + ChildContainerTextPadding + NameTextHeight;

									// Clip the text to the visual's rectangle, with some extra inner padding to avoid overlapping the visual's border
									FSlateDrawElement::MakeText(
										OutDrawElements,
										LayerId,
										AllottedGeometry.ToOffsetPaintGeometry( ShadowOffset + FVector2D( TextX, TextY ) ),
										BlendedVisual.NodeData->Name2,
										BlendedVisual.Name2Font,
										DrawEffects,
										InWidgetStyle.GetColorAndOpacityTint() * VisualTextColor );
								}

								// Center text
								if( !BlendedVisual.NodeData->CenterText.IsEmpty() && BlendedVisual.NodeData->IsLeafNode() )
								{
									// If the visual is smaller than the text we're going to draw, try using a smaller font
									const FSlateFontInfo* BlendedVisualCenterTextFont = &BlendedVisual.CenterTextFont;
									const FVector2D FullTextSize = FontMeasureService->Measure( BlendedVisual.NodeData->CenterText, *BlendedVisualCenterTextFont );
									if( ScreenSpaceVisualSize.X < FullTextSize.X || ScreenSpaceVisualSize.Y < FullTextSize.Y )	// @todo treemap: assumption that NameFont will be smaller than CenterText font
									{
										BlendedVisualCenterTextFont = &BlendedVisual.NameFont;
									}

									const FVector2D TextSize = FontMeasureService->Measure( BlendedVisual.NodeData->CenterText, *BlendedVisualCenterTextFont );
									const auto TextX = VisualPosition.X + FMath::Max( ChildContainerTextPadding, (float)BlendedVisual.Size.X * 0.5f - TextSize.X * 0.5f );	// Clamp to left edge if cropped, so the user can at least read the beginning
									const auto TextY = VisualPosition.Y + FMath::Max( ChildContainerTextPadding + NameTextHeight + Name2TextHeight, (float)BlendedVisual.Size.Y * 0.5f - TextSize.Y * 0.5f );	// Clamp to bottom of first line of text if cropped

									// Clip the text to the visual's rectangle, with some extra inner padding to avoid overlapping the visual's border
									FSlateDrawElement::MakeText(
										OutDrawElements,
										LayerId,
										AllottedGeometry.ToOffsetPaintGeometry( ShadowOffset + FVector2D( TextX, TextY ) ),
										BlendedVisual.NodeData->CenterText,
										*BlendedVisualCenterTextFont,
										DrawEffects,
										InWidgetStyle.GetColorAndOpacityTint() * VisualTextColor );
								}

								OutDrawElements.PopClip();
							}
						}
					}
				}
			}
		}
	}


	if( DraggingVisual != nullptr && DragVisualDistance >= STreeMapDefs::MinCursorDistanceForDraggingVisual )
	{
		const FVector2D DragStartCursorPos = RelativeDragStartMouseCursorPos;
		const FVector2D NewCursorPos = RelativeMouseCursorPos;

		const FVector2D SplineStart = DragStartCursorPos;
		const FVector2D SplineEnd = NewCursorPos;

		const FVector2D SplineStartDir = ( DragStartCursorPos - ( DraggingVisual->Position + DraggingVisual->Size * 0.5f ) );	// @todo treemap: Point away from the center of the dragged visual
		const FVector2D SplineEndDir = FVector2D( 0.0f, 200.0f );	// @todo treemap: Probably needs better customization support

		// @todo treemap: Draw line in red if drop won't work?

		// Draw two passes, the first one is an drop shadow
		for( auto SplineLayerIndex = 0; SplineLayerIndex < 2; ++SplineLayerIndex )
		{
			++LayerId;

			const float ShadowOpacity = 0.5f;
			const float SplineThickness = ( SplineLayerIndex == 0 ) ? 5.0f : 4.0f;
			const auto ShadowOffset = ( SplineLayerIndex == 0 ) ? FVector2D( -1.0f, 1.0f ) : FVector2D::ZeroVector;
			const auto SplineColorScale = ( SplineLayerIndex == 0 ) ? FLinearColor( 0.0f, 0.0f, 0.0f, ShadowOpacity ) : FLinearColor::White;

			FSlateDrawElement::MakeSpline(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry( FVector2f( 1.0, 1.0f ), FSlateLayoutTransform(ShadowOffset) ),
				SplineStart,
				SplineStartDir,
				SplineEnd,
				SplineEndDir,
				SplineThickness,
				DrawEffects,
				InWidgetStyle.GetColorAndOpacityTint() * DraggingVisual->Color * SplineColorScale );
		}
	}


	return LayerId;
}

FVector2D STreeMap::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	// TreeMap widgets have no desired size -- their size is always determined by their container
	return FVector2D::ZeroVector;
}

FReply STreeMap::OnMouseMove( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	FReply Reply = FReply::Unhandled();

	// Update window-relative cursor position
	RelativeMouseCursorPos = InMyGeometry.AbsoluteToLocal( InMouseEvent.GetScreenSpacePosition() );

	// Clear current hover
	MouseOverVisual = nullptr;

	// Don't hover while transitioning.  It looks bad!
	if( !IsNavigationTransitionActive() )
	{
		// Figure out which visual the cursor is over
		FTreeMapNodeVisualInfo* NodeVisualUnderCursor = FindNodeVisualUnderCursor( InMyGeometry, InMouseEvent.GetScreenSpacePosition() );
		if( NodeVisualUnderCursor != nullptr )
		{
			// Mouse is over a visual
			MouseOverVisual = NodeVisualUnderCursor;
		}

		Reply = FReply::Handled();
	}

	if( DraggingVisual != nullptr )
	{
		DragVisualDistance += InMouseEvent.GetCursorDelta().Size();
	}

	return Reply;
}

void STreeMap::OnMouseLeave( const FPointerEvent& InMouseEvent )
{
	MouseOverVisual = nullptr;
}

FReply STreeMap::OnMouseButtonDown( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	FReply Reply = FReply::Unhandled();
	if( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		// Wait until we've finished animating a transition
		if( !IsNavigationTransitionActive() )
		{
			if( AllowEditing.Get() )
			{
				// Figure out what was clicked on
				FTreeMapNodeVisualInfo* NodeVisualUnderCursor = FindNodeVisualUnderCursor( InMyGeometry, InMouseEvent.GetScreenSpacePosition() );
				if( NodeVisualUnderCursor != nullptr )
				{
					// Mouse was pressed on a node
					DraggingVisual = NodeVisualUnderCursor;
					DragVisualDistance = 0.0f;
					RelativeDragStartMouseCursorPos = InMyGeometry.AbsoluteToLocal( InMouseEvent.GetScreenSpacePosition() );

					Reply = FReply::Handled();
				}
			}
		}
	}

	return Reply;
}

FReply STreeMap::OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	FReply Reply = FReply::Unhandled();
	FTreeMapNodeVisualInfo* NodeVisualUnderCursor = FindNodeVisualUnderCursor(InMyGeometry, InMouseEvent.GetScreenSpacePosition());
	if( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		auto* DroppedVisual = DraggingVisual;
		DraggingVisual = nullptr;

		if( DroppedVisual != nullptr && DragVisualDistance >= STreeMapDefs::MinCursorDistanceForDraggingVisual )
		{
			// Wait until we've finished animating a transition
			if( !IsNavigationTransitionActive() && NodeVisualUnderCursor != nullptr )
			{
				// The dropped node will become a child of the node we dropped onto
				auto NewParentNode = NodeVisualUnderCursor->NodeData->AsShared();
				auto DroppedNode = DroppedVisual->NodeData->AsShared();

				// Reparent it!
				ReparentNode( DroppedNode, NewParentNode );

				Reply = FReply::Handled();
			}
		}
		else if( NodeVisualUnderCursor != nullptr )
		{
			if( AllowEditing.Get() )
			{
				// Start renaming!
				const bool bIsNewNode = false;
				StartRenamingNode( InMyGeometry, NodeVisualUnderCursor->NodeData->AsShared(), NodeVisualUnderCursor->Position, bIsNewNode );

				Reply = FReply::Handled();
			}
		}

		DragVisualDistance = 0.0f;
	}
	else if( InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		if (OnTreeMapNodeRightClicked.IsBound())
		{
			if (NodeVisualUnderCursor != nullptr)
			{
				OnTreeMapNodeRightClicked.Execute(*NodeVisualUnderCursor->NodeData, InMouseEvent);
			}
		}
		else
		{
			// Show a pop-up menu!
			ShowOptionsMenuAt(InMouseEvent);
		}
	}
	else if ( InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton )
	{
		// Back button
		if (ZoomOut() && !Reply.IsEventHandled())
		{
			Reply = FReply::Handled();
		}
	}
	else if ( InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2 )
	{
		if (NodeVisualUnderCursor != nullptr)
		{
			// Do zoom behavior
			const bool bShouldPlayTransition = true;
			SetTreeRoot(NodeVisualUnderCursor->NodeData->AsShared(), bShouldPlayTransition);
		}
	}
	
	return Reply;
}

FReply STreeMap::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	FReply Reply = FReply::Unhandled();

	if( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		// Wait until we're done transitioning before allowing another transition
		if( !IsNavigationTransitionActive() )
		{
			// Figure out what was clicked on
			const FTreeMapNodeVisualInfo* NodeVisualUnderCursor = FindNodeVisualUnderCursor( InMyGeometry, InMouseEvent.GetScreenSpacePosition() );
			if( NodeVisualUnderCursor != nullptr )
			{
				// Double-clicked on a tree map visual!  Check to see if we were asked to customize how double-click is handled.
				if( OnTreeMapNodeDoubleClicked.IsBound() )
				{
					OnTreeMapNodeDoubleClicked.Execute( *NodeVisualUnderCursor->NodeData, InMouseEvent );
				}
				else
				{
					// Double-click was not overridden, so just do our default thing and re-root the tree directly on the node that was double-clicked on
					
					// Re-root the tree
					const bool bShouldPlayTransition = true;
					SetTreeRoot( NodeVisualUnderCursor->NodeData->AsShared(), bShouldPlayTransition );
				}

				Reply = FReply::Handled();
			}
		}
	}

	return Reply;
}


FReply STreeMap::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();

	// Don't zoom in or out while already transitioning.  It feels too frenetic.
	if( !IsNavigationTransitionActive() )
	{
		if( MouseEvent.GetWheelDelta() > 0 )
		{
			// Figure out what was clicked on
			const FTreeMapNodeVisualInfo* NodeVisualUnderCursor = FindNodeVisualUnderCursor( MyGeometry, MouseEvent.GetScreenSpacePosition() );
			if( NodeVisualUnderCursor != nullptr )
			{
				// From the node that was scrolled over, visits nodes upward until we find one whose parent is our current active root
				FTreeMapNodeDataPtr NextNode = NodeVisualUnderCursor->NodeData->AsShared();
				do
				{
					FTreeMapNodeDataPtr NodeParent;
					if( NextNode->Parent != nullptr )
					{
						NodeParent = NextNode->Parent->AsShared();
					}

					if( NodeParent == ActiveRootTreeMapNode )
					{
						break;
					}
					NextNode = NodeParent;
				}
				while( NextNode.IsValid() );

				// Zoom in one level
				if( NextNode.IsValid() )
				{
					const bool bShouldPlayTransition = true;
					SetTreeRoot( NextNode.ToSharedRef(), bShouldPlayTransition );

					if( !Reply.IsEventHandled() )
					{
						Reply = FReply::Handled();
					}
				}
			}
		}
		else if( MouseEvent.GetWheelDelta() < 0 )
		{
			// Zoom out one level
			if( ActiveRootTreeMapNode->Parent != nullptr )
			{
				const bool bShouldPlayTransition = true;
				SetTreeRoot( ActiveRootTreeMapNode->Parent->AsShared(), bShouldPlayTransition );

				if( !Reply.IsEventHandled() )
				{
					Reply = FReply::Handled();
				}
			}
		}
	}

	return Reply;
}


void STreeMap::SetTreeRoot( const FTreeMapNodeDataRef& NewRoot, const bool bShouldPlayTransition )
{
	if( ActiveRootTreeMapNode != NewRoot )
	{
		ActiveRootTreeMapNode = NewRoot;

		// Freshen the visualization data for the node since it may be stale after being copied off the undo stack
		RebuildTreeMap( bShouldPlayTransition );
	}
}


FTreeMapNodeDataPtr STreeMap::GetTreeRoot() const
{
	return ActiveRootTreeMapNode;
}


bool STreeMap::CanZoomOut() const
{
	if (ActiveRootTreeMapNode.IsValid() && ActiveRootTreeMapNode->Parent != nullptr)
	{
		return true;
	}
	return false;
}


bool STreeMap::ZoomOut()
{
	if (!CanZoomOut())
	{
		return false;
	}

	const bool bShouldPlayTransition = true;
	SetTreeRoot(ActiveRootTreeMapNode->Parent->AsShared(), bShouldPlayTransition);

	return true;
}


void STreeMap::RebuildTreeMap( const bool bShouldPlayTransition )
{
	// Stop renaming anything.  We don't want pop-up windows persisting during the transition
	StopRenamingNode();

	// Keep track of the last tree
	LastTreeMap = TreeMap;
	LastCachedNodeVisuals = CachedNodeVisuals;
	if( bShouldPlayTransition )
	{
		NavigateAnimationCurve.Play( AsShared() );
	}
	else
	{
		NavigateAnimationCurve.JumpToEnd();
	}

	// Kill the tree map so that it will be regenerated
	TreeMap.Reset();
	CachedNodeVisuals.Reset();
	DraggingVisual = nullptr;
	MouseOverVisual = nullptr;

	// Update ActiveRootTreeMapNode
	FTreeMapNodeDataPtr NewActiveRoot = FindNodeInCopiedTree(ActiveRootTreeMapNode.ToSharedRef(), TreeMapNodeData.ToSharedRef(), TreeMapNodeData.ToSharedRef());

	if (NewActiveRoot.IsValid())
	{
		ActiveRootTreeMapNode = NewActiveRoot;
	}
	else
	{
		ActiveRootTreeMapNode = TreeMapNodeData;
	}

	// Customize the size and coloring of the source nodes before we rebuild it
	ApplyVisualizationToNodes( ActiveRootTreeMapNode.ToSharedRef() );
}


FTreeMapNodeVisualInfo* STreeMap::FindNodeVisualUnderCursor( const FGeometry& MyGeometry, const FVector2D& ScreenSpaceCursorPosition )
{
	if( TreeMap.IsValid() )
	{
		const FVector2D LocalCursorPosition = MyGeometry.AbsoluteToLocal( ScreenSpaceCursorPosition );

		// NOTE: Iterating backwards so that child-most nodes are checked first
		for( auto VisualIndex = CachedNodeVisuals.Num() - 1; VisualIndex >= 0; --VisualIndex )
		{
			auto& Visual = CachedNodeVisuals[ VisualIndex ];

			// We only allow interactive visuals to be moused over
			if( Visual.bIsInteractive )
			{
				const auto VisualRect = FSlateRect( Visual.Position, Visual.Position + Visual.Size );
				if( VisualRect.ContainsPoint( LocalCursorPosition ) )
				{
					return &Visual;
				}
			}
		}
	}

	return nullptr;
}


bool STreeMap::IsNavigationTransitionActive() const
{
	return NavigateAnimationCurve.IsPlaying();
}


void STreeMap::TakeUndoSnapshot()
{
	// If we've already undone some state, then we'll remove any undo state beyond the level that
	// we've already undone up to.
	if( CurrentUndoLevel != INDEX_NONE )
	{
		UndoStates.RemoveAt( CurrentUndoLevel, UndoStates.Num() - CurrentUndoLevel );

		// Reset undo level as we haven't undone anything since this latest undo
		CurrentUndoLevel = INDEX_NONE;
	}

	// Take an Undo snapshot before we change anything
	UndoStates.Add( this->TreeMapNodeData->CopyNodeRecursively() );

	// If we've hit the undo limit, then delete previous entries
	const int32 MaxUndoLevels = 200;	// @todo treemap custom: Make customizable in the settings of the widget?
	if( UndoStates.Num() > MaxUndoLevels )
	{
		UndoStates.RemoveAt( 0 );
	}
}


FTreeMapNodeDataPtr STreeMap::FindNodeInCopiedTree( const FTreeMapNodeDataRef& NodeToFind, const FTreeMapNodeDataRef& OriginalNode, const FTreeMapNodeDataRef& CopiedNode ) const
{
	if( NodeToFind.Get() == OriginalNode.Get() )
	{
		return CopiedNode;
	}

	for( int32 ChildNodeIndex = 0; ChildNodeIndex < OriginalNode->Children.Num(); ++ChildNodeIndex )
	{
		const auto& OriginalChildNode = OriginalNode->Children[ ChildNodeIndex ];
		const auto& CopiedChildNode = CopiedNode->Children[ ChildNodeIndex ];

		const auto& FoundMatchingCopy = FindNodeInCopiedTree( NodeToFind, OriginalChildNode.ToSharedRef(), CopiedChildNode.ToSharedRef() );
		if( FoundMatchingCopy.IsValid() )
		{
			return FoundMatchingCopy;
		}
	}

	return nullptr;
}


void STreeMap::Undo()
{
	if( UndoStates.Num() > 0 )
	{
		// Restore from undo state
		int32 UndoStateIndex;
		if( CurrentUndoLevel == INDEX_NONE )
		{
			// We haven't undone anything since the last time a new undo state was added
			UndoStateIndex = UndoStates.Num() - 1;

			// Store an undo state for the current state (before undo was pressed)
			TakeUndoSnapshot();
		}
		else
		{
			// Move down to the next undo level
			UndoStateIndex = CurrentUndoLevel - 1;
		}

		// Is there anything else to undo?
		if( UndoStateIndex >= 0 )
		{
			// Undo from history
			{
				const FTreeMapNodeDataRef NewTreeMapNodeData = UndoStates[ UndoStateIndex ]->CopyNodeRecursively();
				const FTreeMapNodeDataRef NewActiveRoot = FindNodeInCopiedTree( ActiveRootTreeMapNode.ToSharedRef(), TreeMapNodeData.ToSharedRef(), NewTreeMapNodeData ).ToSharedRef();
				TreeMapNodeData = NewTreeMapNodeData;

				const bool bShouldPlayTransition = true;		// @todo treemap: A bit worried about volatility of LastTreeMap node refs here
				SetTreeRoot( NewActiveRoot, bShouldPlayTransition );
			}

			CurrentUndoLevel = UndoStateIndex;
		}
	}
}


void STreeMap::Redo()
{
	// Is there anything to redo?  If we've haven't tried to undo since the last time new
	// undo state was added, then CurrentUndoLevel will be INDEX_NONE
	if( CurrentUndoLevel != INDEX_NONE )
	{
		const int32 NextUndoLevel = CurrentUndoLevel + 1;
		if( UndoStates.Num() > NextUndoLevel )
		{
			// Restore from undo state
			{
				const FTreeMapNodeDataRef NewTreeMapNodeData = UndoStates[ NextUndoLevel ]->CopyNodeRecursively();
				const FTreeMapNodeDataRef NewActiveRoot = FindNodeInCopiedTree( ActiveRootTreeMapNode.ToSharedRef(), TreeMapNodeData.ToSharedRef(), NewTreeMapNodeData ).ToSharedRef();
				TreeMapNodeData = NewTreeMapNodeData;

				const bool bShouldPlayTransition = true;	// @todo treemap: A bit worried about volatility of LastTreeMap node refs here
				SetTreeRoot( NewActiveRoot, bShouldPlayTransition );
			}

			CurrentUndoLevel = NextUndoLevel;

			if( UndoStates.Num() <= CurrentUndoLevel + 1 )
			{
				// We've redone all available undo states
				CurrentUndoLevel = INDEX_NONE;

				// Pop last undo state that we created on initial undo
				UndoStates.RemoveAt( UndoStates.Num() - 1 );
			}
		}
	}
}


void STreeMap::ReparentNode( const FTreeMapNodeDataRef DroppedNode, const FTreeMapNodeDataRef NewParentNode )
{
	// Can't reparent the tree root node
	if( DroppedNode != TreeMapNodeData )
	{
		struct Local
		{
			/** Checks to see if the specified Node is a child of 'PossibleParent' at any level */
			static bool IsMyParent( const FTreeMapNodeDataRef& Node, const FTreeMapNodeDataRef& PossibleParent )
			{
				if( Node == PossibleParent )
				{
					return true;
				}

				if( PossibleParent->Parent != nullptr )
				{
					return IsMyParent( Node, PossibleParent->Parent->AsShared() );
				}

				return false;
			}
		};

		// Can't parent owning nodes to their children
		if( !Local::IsMyParent( DroppedNode, NewParentNode ) )
		{
			// Can't reparent to self
			if( NewParentNode != DroppedNode )
			{
				// Only if parent has changed
				if( &NewParentNode.Get() != DroppedNode->Parent )
				{
					// Take an Undo snapshot before we change anything
					TakeUndoSnapshot();

					if( DroppedNode->Parent != nullptr )
					{
						DroppedNode->Parent->Children.RemoveSingle( DroppedNode );
					}
					NewParentNode->Children.Add( DroppedNode );
					DroppedNode->Parent = &NewParentNode.Get();

					// If we container node became a leaf node, we need to make sure it has a valid size set
					// @todo treemap: This seems a bit... weird.  It won't reverse either, if you put the node back (save with overridden size!)  Maybe we need a bool for "auto size"=true.  OR, have a separate size for Node vs. Leaf?
					if( DroppedNode->Size == 0.0f && DroppedNode->IsLeafNode() )
					{
						DroppedNode->Size = 1.0f;
					}

					// Invalidate the tree
					const bool bShouldPlayTransition = true;
					RebuildTreeMap( bShouldPlayTransition );

					// After we have a new tree, play a highlight effect on the reparented node so the user can see where it ended
					// up in the tree.  Tree map layout can be fairly volatile after a parenting change.
					HighlightPulseNode = DroppedNode;
					HighlightPulseStartTime = FSlateApplication::Get().GetCurrentTime();
				}
			}
		}
	}
}


FReply STreeMap::DeleteHoveredNode()
{
	FReply Reply = FReply::Unhandled();

	// Wait until we've finished animating a transition
	if( !IsNavigationTransitionActive() )
	{
		if( MouseOverVisual != nullptr )
		{
			// Only non-root nodes can be deleted
			auto NodeToDelete = MouseOverVisual->NodeData->AsShared();
			if( NodeToDelete->Parent != nullptr )
			{
				// Don't allow the current active tree root to be deleted.  The user should zoom out first!
				if( NodeToDelete != ActiveRootTreeMapNode )
				{
					TakeUndoSnapshot();

					// Delete the node
					{
						NodeToDelete->Parent->Children.RemoveSingle( NodeToDelete );
						NodeToDelete->Parent = nullptr;

						// Note:  NodeToDelete will actually be deleted when it goes out of scope later in this function (shared pointer), but it is already removed from our tree
					}

					const bool bShouldPlayTransition = true;
					RebuildTreeMap( bShouldPlayTransition );

					Reply = FReply::Handled();
				}
			}
		}
	}

	return Reply;
}


FReply STreeMap::InsertNewNodeAsChildOfHoveredNode( const FGeometry& MyGeometry )
{
	FReply Reply = FReply::Unhandled();

	// Wait until we've finished animating a transition
	if( !IsNavigationTransitionActive() )
	{
		if( MouseOverVisual != nullptr )
		{
			auto ParentNode = MouseOverVisual->NodeData->AsShared();

			// Allocate the new node but don't add it to the tree yet.  We want the user to give the node a name first!

			// NOTE: Ownership of this node will be transferred to StartRenamingNode, where it will be referenced
			// in the delegate callback for the editable text change commit handler.  If the user opts to not type anything, the
			// node will be deleted instead of being added to the tree.
			FTreeMapNodeDataRef NewNode( new FTreeMapNodeData() );
			NewNode->Size = 1.0f;	// Leaf nodes always get a size of 1.0 for now
			NewNode->Parent = MouseOverVisual->NodeData;

			// Start editing the new node before we insert it
			const bool bIsNewNode = true;
			StartRenamingNode( MyGeometry, NewNode, MouseOverVisual->Position, bIsNewNode );

			Reply = FReply::Handled();
		}
	}

	return Reply;
}


bool STreeMap::SupportsKeyboardFocus() const
{
	return true;
}


FReply STreeMap::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyboardEvent )
{
	FReply Reply = FReply::Unhandled();

	const FKey Key = InKeyboardEvent.GetKey();

	if( AllowEditing.Get() )
	{
		if( Key == EKeys::Delete )
		{
			// If the cursor is over a node, delete it!
			DeleteHoveredNode();
			Reply = FReply::Handled();
		}
		else if( Key == EKeys::Insert )
		{
			// If the cursor is over a node, insert a new node as a child!
			InsertNewNodeAsChildOfHoveredNode( MyGeometry );
			Reply = FReply::Handled();
		}
		else if( Key == EKeys::Z && InKeyboardEvent.IsControlDown() )
		{
			// Undo
			Undo();
			Reply = FReply::Handled();
		}
		else if( Key == EKeys::Y && InKeyboardEvent.IsControlDown() )
		{
			// Redo
			Redo();
			Reply = FReply::Handled();
		}
	}

	return Reply;
}


void STreeMap::RenamingNode_OnTextCommitted( const FText& NewText, ETextCommit::Type, TSharedRef<FTreeMapNodeData> NodeToRename )
{
	TSharedPtr<SWidget> RenamingNodeWidgetPinned( RenamingNodeWidget.Pin() );
	if( RenamingNodeWidgetPinned.IsValid() )
	{
		// Kill the window after the text has been committed
		TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( RenamingNodeWidgetPinned.ToSharedRef() ).ToSharedRef();

		RenamingNodeWidget.Reset();	// Avoid reentrancy
		RenamingNodeData.Reset();
		FSlateApplication::Get().RequestDestroyWindow( ParentWindow );


		// Make sure new name is OK
		if( NewText.ToString() != NodeToRename->Name &&
			!NewText.IsEmpty() )
		{
			// Store undo state
			TakeUndoSnapshot();

			// Rename it!
			NodeToRename->Name = NewText.ToString();

			if( bIsNamingNewNode )
			{
				// Add the new node to the tree
				TakeUndoSnapshot();

				// Insert the new node
				{
					auto ParentNode = NodeToRename->Parent->AsShared();
					ParentNode->Children.Add( NodeToRename );
				}

				const bool bShouldPlayTransition = true;
				RebuildTreeMap( bShouldPlayTransition );
			}
			else
			{
				// NOTE: No refresh needed, as node labels are pulled directly from nodes
			}
		}
	}
}



void STreeMap::StartRenamingNode( const FGeometry& MyGeometry, const FTreeMapNodeDataRef& NodeData, const FVector2D& RelativePosition, const bool bIsNewNode )
{
	TSharedRef<SBorder> RenamerWidget = SNew( SBorder );

	TSharedRef<SEditableTextBox> EditableText = 
		SNew( SEditableTextBox )
			.Text( FText::FromString( NodeData->Name) )
			.SelectAllTextWhenFocused( true )
			.RevertTextOnEscape( true )
			.MinDesiredWidth( 100 )
			.OnTextCommitted( this, &STreeMap::RenamingNode_OnTextCommitted, NodeData->AsShared() )
		;
	RenamerWidget->SetContent( EditableText );

	RenamingNodeWidget = RenamerWidget;
	RenamingNodeData = NodeData;
	bIsNamingNewNode = bIsNewNode;

	const bool bFocusImmediately = true;
	FSlateApplication::Get().PushMenu( AsShared(), FWidgetPath(), RenamerWidget, MyGeometry.LocalToAbsolute( RelativePosition ), FPopupTransitionEffect::None, bFocusImmediately );

	// Focus the text box right after we spawn it so that the user can start typing
	FSlateApplication::Get().SetKeyboardFocus( EditableText, EFocusCause::SetDirectly );
}



void STreeMap::StopRenamingNode()
{
	TSharedPtr<SWidget> RenamingNodeWidgetPinned( RenamingNodeWidget.Pin() );
	if( RenamingNodeWidgetPinned.IsValid() )
	{
		// Kill the window after the text has been committed
		TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( RenamingNodeWidgetPinned.ToSharedRef() ).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow( ParentWindow );
	}
}


void STreeMap::ApplyVisualizationToNodes( const FTreeMapNodeDataRef& Node )
{
	const FLinearColor RootDefaultColor = FLinearColor( 0.125f, 0.125f, 0.125f );	// @todo treemap custom: Make configurable in the STreeMap settings?
	int32 TreeDepth = 0;
	ApplyVisualizationToNodesRecursively( Node, RootDefaultColor, TreeDepth );
}


void STreeMap::ApplyVisualizationToNodesRecursively( const FTreeMapNodeDataRef& Node, const FLinearColor& DefaultColor, const int32 TreeDepth )
{
	// Select default color saturation based on tree depth
	FLinearColor MyDefaultColor = DefaultColor;
	auto HSV = MyDefaultColor.LinearRGBToHSV();
	const float SaturationReductionPerDepthLevel = 0.05f;	// @todo treemap custom: Make configurable in the tree map settings?  Calculate tree depth?
	const float MinAllowedSaturation = 0.1f;
	HSV.G = FMath::Max( MinAllowedSaturation, HSV.G - TreeDepth * SaturationReductionPerDepthLevel );
	MyDefaultColor = HSV.HSVToLinearRGB();


	// Size
	{
		if( SizeNodesByAttribute.IsValid() )
		{
			FTreeMapAttributeDataPtr* AttributeDataPtr = Node->Attributes.Find( SizeNodesByAttribute->Name );
			if( AttributeDataPtr != nullptr )
			{
				const FTreeMapAttributeData& AttributeData = *AttributeDataPtr->Get();

				// Make sure the data value is in range
				if( ensure( SizeNodesByAttribute->Values.Contains( AttributeData.Value ) ) )
				{
					// @todo treemap: This doesn't work well with container nodes.  Our range of sizes doesn't go big enough to compete with cumulative sizes!
					Node->Size = SizeNodesByAttribute->Values[ AttributeData.Value ]->NodeSize;
				}
				else
				{
					// Invalid attribute data!
					Node->Size = Node->IsLeafNode() ? SizeNodesByAttribute->DefaultValue->NodeSize : 0.0f;	// Leaf nodes get a default size, but container nodes use auto-sizing
				}
			}
			else
			{
				// The node doesn't have this attribute on it.  Use default.
				Node->Size = Node->IsLeafNode() ? SizeNodesByAttribute->DefaultValue->NodeSize : 0.0f;	// Leaf nodes get a default size, but container nodes use auto-sizing
			}
		}

		else
		{
			// @todo treemap: Really we want this to "restore the original size set by the user", not make up new defaults.  Disabled for now.

			// Default size
			// Node->Size = Node->IsLeafNode() ? 1.0f : 0.0f;	// Leaf nodes get a default size, but container nodes use auto-sizing
		}
	}


	// Color
	{
		if( ColorNodesByAttribute.IsValid() )
		{
			FTreeMapAttributeDataPtr* AttributeDataPtr = Node->Attributes.Find( ColorNodesByAttribute->Name );
			if( AttributeDataPtr != nullptr )
			{
				const FTreeMapAttributeData& AttributeData = *AttributeDataPtr->Get();

				// Make sure the data value is in range
				if( ensure( ColorNodesByAttribute->Values.Contains( AttributeData.Value ) ) )
				{
					Node->Color = ColorNodesByAttribute->Values[ AttributeData.Value ]->NodeColor;
				}
				else
				{
					// Invalid attribute data!
					Node->Color = ColorNodesByAttribute->DefaultValue->NodeColor;
				}
			}
			else
			{
				// The node doesn't have this attribute on it.  Use default.
				Node->Color = ColorNodesByAttribute->DefaultValue->NodeColor;
			}
		}
		else
		{
			// Default color
			Node->Color = MyDefaultColor;
		}
	}



	for( int32 ChildIndex = 0; ChildIndex < Node->Children.Num(); ++ChildIndex )
	{
		const auto& ChildNode = Node->Children[ ChildIndex ];

		// Make up a distinct color for all of the root's top level nodes
		FLinearColor ChildColor = MyDefaultColor;
		if( TreeDepth == 0 )
		{
			// Choose a hue evenly spaced across the spectrum
			float ColorHue = 360.0f * (float)( ChildIndex + 1 ) / (float)Node->Children.Num();
			auto ChildColorHSV = FLinearColor::White;
			ChildColorHSV.R = ColorHue;
			ChildColorHSV.G = 1.0f;	// Full saturation!
			ChildColor = ChildColorHSV.HSVToLinearRGB();
		}

		ApplyVisualizationToNodesRecursively( ChildNode.ToSharedRef(), ChildColor, TreeDepth + 1 );
	}
}

void STreeMap::ShowOptionsMenuAt(const FPointerEvent& InMouseEvent)
{
	FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
	const FVector2D& ScreenSpacePosition = InMouseEvent.GetScreenSpacePosition();

	ShowOptionsMenuAtInternal(ScreenSpacePosition, WidgetPath);
}

void STreeMap::ShowOptionsMenuAtInternal(const FVector2D& ScreenSpacePosition, const FWidgetPath& WidgetPath)
{
	struct Local
	{
		static void MakeEditNodeAttributeMenu( FMenuBuilder& MenuBuilder, FTreeMapNodeDataPtr EditingNode, TSharedPtr< FTreeMapAttribute > Attribute, STreeMap* Self )
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT( "RemoveAttribute", "Not Set" ),
				FText(),	// No tooltip (intentional)
				FSlateIcon(),	// Icon
				FUIAction(
				FExecuteAction::CreateStatic( &Local::EditNodeAttribute_Execute, EditingNode, Attribute, TSharedPtr< FTreeMapAttributeValue >(), Self ),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic( &Local::EditNodeAttribute_IsChecked, EditingNode, Attribute, TSharedPtr< FTreeMapAttributeValue >(), Self ) ),
				NAME_None,	// Extension point
				EUserInterfaceActionType::ToggleButton );

			MenuBuilder.AddMenuSeparator();

			// @todo treemap: These probably should be sorted rather than hash order
			for( auto HashIter( Attribute->Values.CreateConstIterator() ); HashIter; ++HashIter )
			{
				FTreeMapAttributeValuePtr Value = HashIter.Value();

				MenuBuilder.AddMenuEntry(
					FText::FromName( Value->Name ),
					FText(),	// No tooltip (intentional)
					FSlateIcon(),	// Icon
					FUIAction(
					FExecuteAction::CreateStatic( &Local::EditNodeAttribute_Execute, EditingNode, Attribute, Value, Self ),
					FCanExecuteAction(),
					FIsActionChecked::CreateStatic( &Local::EditNodeAttribute_IsChecked, EditingNode, Attribute, Value, Self ) ),
					NAME_None,	// Extension point
					EUserInterfaceActionType::ToggleButton );
			}
		}


		static void EditNodeAttribute_Execute( FTreeMapNodeDataPtr EditingNode, TSharedPtr< FTreeMapAttribute > Attribute, TSharedPtr< FTreeMapAttributeValue > Value, STreeMap* Self )
		{
			bool bAnyChanges = false;
			if( Value.IsValid() )
			{
				if( !EditingNode->Attributes.Contains( Attribute->Name ) )
				{
					EditingNode->Attributes.Add( Attribute->Name, MakeShareable( new FTreeMapAttributeData( Value->Name ) ) );
					bAnyChanges = true;
				}
				else if( EditingNode->Attributes[ Attribute->Name ]->Value != Value->Name )
				{
					EditingNode->Attributes[ Attribute->Name ]->Value = Value->Name;
					bAnyChanges = true;
				}
				else
				{
					// Nothing changed
				}
			}
			else
			{
				// Clearing attribute
				if( EditingNode->Attributes.Contains( Attribute->Name ) )
				{
					EditingNode->Attributes.Remove( Attribute->Name );
					bAnyChanges = true;
				}
				else
				{
					// Nothing changed
				}
			}

			// Has anything changed?
			if( bAnyChanges )
			{
				const bool bShouldPlayTransition = true;
				Self->RebuildTreeMap( bShouldPlayTransition );
			}
		}


		static bool EditNodeAttribute_IsChecked( FTreeMapNodeDataPtr EditingNode, TSharedPtr< FTreeMapAttribute > Attribute, TSharedPtr< FTreeMapAttributeValue > Value, STreeMap* Self )
		{
			if( Value.IsValid() )
			{
				return EditingNode->Attributes.Contains( Attribute->Name ) && EditingNode->Attributes[ Attribute->Name ]->Value == Value->Name;
			}
			else
			{
				return !EditingNode->Attributes.Contains( Attribute->Name );
			}
		}


		static void SizeByAttribute_Execute( TSharedPtr< FTreeMapAttribute > Attribute, STreeMap* Self )
		{
			// Has anything changed?
			if( Self->SizeNodesByAttribute != Attribute )
			{
				// Apply the new visualization!
				Self->SizeNodesByAttribute = Attribute;

				const bool bShouldPlayTransition = true;
				Self->RebuildTreeMap( bShouldPlayTransition );
			}
		}


		static bool SizeByAttribute_IsChecked( TSharedPtr< FTreeMapAttribute > Attribute, STreeMap* Self )
		{
			return Self->SizeNodesByAttribute == Attribute;
		}


		static void MakeSizeByAttributeMenu( FMenuBuilder& MenuBuilder, STreeMap* Self )
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT( "NoSizeByAttribute", "Off" ),
				FText(),	// No tooltip (intentional)
				FSlateIcon(),	// Icon
				FUIAction(
				FExecuteAction::CreateStatic( &Local::SizeByAttribute_Execute, TSharedPtr< FTreeMapAttribute >(), Self ),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic( &Local::SizeByAttribute_IsChecked, TSharedPtr< FTreeMapAttribute >(), Self ) ),
				NAME_None,	// Extension point
				EUserInterfaceActionType::ToggleButton );

			if( Self->Customization.IsValid() )
			{
				MenuBuilder.AddMenuSeparator();

				// @todo treemap: These probably should be sorted rather than hash order
				for( auto HashIter( Self->Customization->GetAttributes().CreateConstIterator() ); HashIter; ++HashIter )
				{
					FTreeMapAttributePtr Attribute = HashIter.Value();

					MenuBuilder.AddMenuEntry(
						FText::FromName( Attribute->Name ),
						FText(),	// No tooltip (intentional)
						FSlateIcon(),	// Icon
						FUIAction(
						FExecuteAction::CreateStatic( &Local::SizeByAttribute_Execute, Attribute, Self ),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic( &Local::SizeByAttribute_IsChecked, Attribute, Self ) ),
						NAME_None,	// Extension point
						EUserInterfaceActionType::ToggleButton );
				}
			}
		}


		static void ColorByAttribute_Execute( TSharedPtr< FTreeMapAttribute > Attribute, STreeMap* Self )
		{
			// Has anything changed?
			if( Self->ColorNodesByAttribute != Attribute )
			{
				// Apply the new visualization!
				Self->ColorNodesByAttribute = Attribute;

				const bool bShouldPlayTransition = true;
				Self->RebuildTreeMap( bShouldPlayTransition );
			}
		}


		static bool ColorByAttribute_IsChecked( TSharedPtr< FTreeMapAttribute > Attribute, STreeMap* Self )
		{
			return Self->ColorNodesByAttribute == Attribute;
		}


		static void MakeColorByAttributeMenu( FMenuBuilder& MenuBuilder, STreeMap* Self )
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT( "NoColorByAttribute", "Off" ),
				FText(),	// No tooltip (intentional)
				FSlateIcon(),	// Icon
				FUIAction(
				FExecuteAction::CreateStatic( &Local::ColorByAttribute_Execute, TSharedPtr< FTreeMapAttribute >(), Self ),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic( &Local::ColorByAttribute_IsChecked, TSharedPtr< FTreeMapAttribute >(), Self ) ),
				NAME_None,	// Extension point
				EUserInterfaceActionType::ToggleButton );

			if( Self->Customization.IsValid() )
			{
				MenuBuilder.AddMenuSeparator();

				// @todo treemap: These probably should be sorted rather than hash order
				for( auto HashIter( Self->Customization->GetAttributes().CreateConstIterator() ); HashIter; ++HashIter )
				{
					FTreeMapAttributePtr Attribute = HashIter.Value();

					MenuBuilder.AddMenuEntry(
						FText::FromName( Attribute->Name ),
						FText(),	// No tooltip (intentional)
						FSlateIcon(),	// Icon
						FUIAction(
						FExecuteAction::CreateStatic( &Local::ColorByAttribute_Execute, Attribute, Self ),
						FCanExecuteAction(),
						FIsActionChecked::CreateStatic( &Local::ColorByAttribute_IsChecked, Attribute, Self ) ),
						NAME_None,	// Extension point
						EUserInterfaceActionType::ToggleButton );
				}
			}
		}
	};

	// Only present a context menu if the tree has been customized
	if( Customization.IsValid() )
	{
		const bool bShouldCloseMenuAfterSelection = true;
		FMenuBuilder OptionsMenuBuilder( bShouldCloseMenuAfterSelection, nullptr );

		if( AllowEditing.Get() && Customization.IsValid() && MouseOverVisual != nullptr )
		{
			// Node editing options
			OptionsMenuBuilder.BeginSection( NAME_None, LOCTEXT( "EditNodeAttributesSection", "Edit Node" ) );
			{
				const auto& EditingNode = MouseOverVisual->NodeData->AsShared();

				for( auto HashIter( Customization->GetAttributes().CreateConstIterator() ); HashIter; ++HashIter )
				{
					FTreeMapAttributePtr Attribute = HashIter.Value();

					OptionsMenuBuilder.AddSubMenu(
						FText::FromName( Attribute->Name ),
						LOCTEXT( "EditAttribute_ToolTip", "Edits this attribute on the node under the curosr." ),
						FNewMenuDelegate::CreateStatic( &Local::MakeEditNodeAttributeMenu, FTreeMapNodeDataPtr( EditingNode ), Attribute, this ) );
				}
			}
			OptionsMenuBuilder.EndSection();
		}

		OptionsMenuBuilder.BeginSection( NAME_None, LOCTEXT( "ChangeLayoutSection", "Layout" ) );
		{
			OptionsMenuBuilder.AddSubMenu( LOCTEXT( "SizeBy", "Size by" ), LOCTEXT( "SizeBy_ToolTip", "Sets which criteria to base the size of the nodes in the tree map on." ), FNewMenuDelegate::CreateStatic( &Local::MakeSizeByAttributeMenu, this ) );
			OptionsMenuBuilder.AddSubMenu( LOCTEXT( "ColorBy", "Color by" ), LOCTEXT( "ColorBy_ToolTip", "Sets which criteria to base the color of the nodes in the tree map on." ), FNewMenuDelegate::CreateStatic( &Local::MakeColorByAttributeMenu, this ) );
		}
		OptionsMenuBuilder.EndSection();

		TSharedRef< SWidget > WindowContent =
			SNew( SBorder )
			[
				OptionsMenuBuilder.MakeWidget()
			];

		const bool bFocusImmediately = false;
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, WindowContent, ScreenSpacePosition, FPopupTransitionEffect::ContextMenu, bFocusImmediately);
	}
}

#undef LOCTEXT_NAMESPACE
