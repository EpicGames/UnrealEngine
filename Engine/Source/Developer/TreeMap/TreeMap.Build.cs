// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TreeMap : ModuleRules
	{
		public TreeMap(ReadOnlyTargetRules Target) : base(Target)
		{
	        PublicDependencyModuleNames.AddRange(new string[] { "Core", "CoreUObject", "SlateCore" });
	        PrivateDependencyModuleNames.AddRange(new string[] { "Slate", "InputCore", "XmlParser" });
		}
	}
}
