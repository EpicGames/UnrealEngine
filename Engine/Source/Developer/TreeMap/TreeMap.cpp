// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "ITreeMap.h"
#include "XmlFile.h"


typedef TSharedPtr<class FTreeMapNode> FTreeMapNodePtr;
typedef TSharedRef<class FTreeMapNode> FTreeMapNodeRef;


/**
 * Rectangle used for tree maps
 */
struct FTreeMapRect
{
	/** Position of the rectangle */
	FVector2D Position;

	/** Dimensions of the rectangle */
	FVector2D Size;


	/** Default constructor */
	FTreeMapRect()
		: Position( FVector2D::ZeroVector ),
		  Size( FVector2D::ZeroVector )
	{
	}
};



/**
 * Single node in a tree map, which may have any number of child nodes, each with their own children and so on
 */
class FTreeMapNode
{

public:

	/** Pointer to the source data for this node */
	FTreeMapNodeDataPtr Data;

	/** List of child nodes */
	TArray< FTreeMapNodePtr > Children;

	/** For leaf nodes, the size of this node.  For non-leaf nodes, the size of all of my child nodes. */
	float Size;

	/** Node rectangle */
	FTreeMapRect Rect;

	/** Node rectangle, with padding applied */
	FTreeMapRect PaddedRect;

	/** Font to use for this node's title */
	FSlateFontInfo NameFont;

	/** Font to use for this node's second line title */
	FSlateFontInfo Name2Font;

	/** Font to use for this node's centered text */
	FSlateFontInfo CenterTextFont;

	/** True if the node is 'interactive'.  That is, we have enough room for a title area and padding for the node to be clicked on */
	bool bIsInteractive;

	/** True if the node is visible at all, if false drawing will skip this entirely */
	bool bIsVisible;

public:

	/** Default constructor for FTreeMapNode */
	FTreeMapNode( const FTreeMapNodeDataRef& InitNodeData );

	/** @return Returns true if this is a leaf node */
	bool IsLeafNode() const
	{
		return Children.Num() == 0;
	}

};



/**
 * Tree map object
 */
class FTreeMap : public ITreeMap
{

public:

	/** Default constructor for FTreeMap */
	FTreeMap( const FTreeMapOptions& Options, const FTreeMapNodeDataRef& RootNodeData );

	/** ITreeMap interface */
	virtual TArray<FTreeMapNodeVisualInfo> GetVisuals() override;


private:

	enum class ESplitDirection
	{
		Horizontal,
		Vertical
	};

	/** Recursively sets up nodes in the tree */
	void AddNodesRecursively( FTreeMapNodePtr& Node, const FTreeMapNodeDataRef& NodeData );

	/** Recursively caches the size of each node.  Leaf nodes get the size of their source node, while non-leaf nodes are set to the total size of all of their children */
	void CalculateNodeSizesRecursively( const FTreeMapNodeRef& Node, float& MaxNodeSize );
	
	/** Scales the specified node and all sub-nodes by the specified amount */
	void ScaleNodesRecursively( const FTreeMapNodeRef& NodeToScale, const float ScaleFactor );

	/** Sets up a node using the standard tree mapping algorithm */
	void MakeStandardTreeNode( const FTreeMapOptions& Options, FTreeMap::ESplitDirection SplitDirection, const FTreeMapNodeRef& Node );

	/** Sets up a node using a squarification method */
	void MakeSquarifiedTreeNode( const FTreeMapOptions& Options, const FTreeMapNodeRef& Node );

	/** Partitions nodes recursively */
	void PartitionNodesRecursively( const FTreeMapOptions& Options, FTreeMap::ESplitDirection SplitDirection, const FTreeMapNodeRef& Node );

	/** Pad all of the nodes in to make room for titles and border */
	void PadNodesRecursively( const FTreeMapOptions& Options, const FTreeMapNodeRef& Node, const int32 TreeDepth );


private:

	/** Root node in the tree map */
	FTreeMapNodePtr RootNode;
};




FTreeMapNode::FTreeMapNode( const FTreeMapNodeDataRef& InitNodeData )
	: Data( InitNodeData ),
	  Children(),
	  Size( 0.0f ),
	  Rect(),
	  PaddedRect(),
	  NameFont(),
	  Name2Font(),
	  CenterTextFont(),
	  bIsInteractive( true )
{
}



FTreeMap::FTreeMap( const FTreeMapOptions& Options, const FTreeMapNodeDataRef& RootNodeData )
{
	AddNodesRecursively( RootNode, RootNodeData );

	// Cache the size of every node
	float MaxNodeSize = 0.0f;
	CalculateNodeSizesRecursively( RootNode.ToSharedRef(), MaxNodeSize );

	// Also fix up the node sizes as we go.  We want the sizes to be proportional to the total display size
	const float DisplaySize = Options.DisplayWidth * Options.DisplayHeight;
	ScaleNodesRecursively( RootNode.ToSharedRef(), DisplaySize / MaxNodeSize );	// @todo treemap perf: Could use a scale factor w/ accessor instead of recursing here

	// The root node has a fixed position and size
	RootNode->Rect.Position = FVector2D::ZeroVector;
	RootNode->Rect.Size = FVector2D( Options.DisplayWidth, Options.DisplayHeight );

	// For regular tree types, we'll choose a "next split direction" that matches the aspect of the display area
	const float DisplayAspect = Options.DisplayWidth / Options.DisplayHeight;
	const bool bIsWiderThanTall = DisplayAspect >= 1.0f;
	const auto SplitDirection = bIsWiderThanTall ? ESplitDirection::Horizontal : ESplitDirection::Vertical;
	PartitionNodesRecursively( Options, SplitDirection, RootNode.ToSharedRef() );

	// Now add space for titles and borders
	const int32 TreeDepth = 0;
	PadNodesRecursively( Options, RootNode.ToSharedRef(), TreeDepth );
}


void FTreeMap::AddNodesRecursively( FTreeMapNodePtr& OutNode, const FTreeMapNodeDataRef& NodeData )
{
	// Setup this node
	OutNode = MakeShareable( new FTreeMapNode( NodeData ) );

	// Add children
	for( const auto& ChildNodeData : NodeData->Children )
	{
		FTreeMapNodePtr ChildNode;
		AddNodesRecursively( ChildNode, ChildNodeData.ToSharedRef() );

		OutNode->Children.Add( ChildNode );
	}
}


void FTreeMap::CalculateNodeSizesRecursively( const FTreeMapNodeRef& Node, float& MaxNodeSize )
{
	// Is this a leaf node?  Leaf nodes will actually determine the size of non-leaf nodes.
	if( Node->IsLeafNode() )
	{
		// NOTE: Size should really always be greater than zero here to get good results, but we don't want to assert.
		Node->Size = Node->Data->Size;
	}
	else
	{
		// Update child node sizes
		float TotalSizeOfChildren = 0.0f;
		float MaxSizeOfChildren = 0.0f;
		for( const auto& ChildNode : Node->Children )
		{
			CalculateNodeSizesRecursively( ChildNode.ToSharedRef(), MaxSizeOfChildren );
			TotalSizeOfChildren += ChildNode->Size;
		}

		// Container node.  If a size was explicitly set, then we'll use that size.
		if( Node->Data->Size > 0.0f )
		{
			Node->Size = Node->Data->Size;

			// Scale the child nodes to fit into the forced container size
			const float ScaleFactor = Node->Size / TotalSizeOfChildren;
			{
				for( const auto& ChildNode : Node->Children )
				{
					ScaleNodesRecursively( ChildNode.ToSharedRef(), ScaleFactor );
				}
			}
		}
		else
		{
			// Create a size for the node by summing it's child node sizes
			Node->Size = TotalSizeOfChildren;
		}

		// Sort our children, largest to smallest
		Node->Children.Sort( []( const FTreeMapNodePtr& A, const FTreeMapNodePtr& B ) { return A->Size > B->Size; } );
	}

	MaxNodeSize = FMath::Max( MaxNodeSize, Node->Size );
}


void FTreeMap::ScaleNodesRecursively( const FTreeMapNodeRef& NodeToScale, const float ScaleFactor )
{
	NodeToScale->Size *= ScaleFactor;

	for( const auto& ChildNode : NodeToScale->Children )
	{
		ScaleNodesRecursively( ChildNode.ToSharedRef(), ScaleFactor );
	}
}


void FTreeMap::MakeStandardTreeNode( const FTreeMapOptions& Options, FTreeMap::ESplitDirection SplitDirection, const FTreeMapNodeRef& Node )
{
	// Standard tree map algorithm.  We alternate between horizontal and vertical packing of children.  All children are packed 
	// into a single row or column.  This makes it fairly easy to see the hierarchical structure of the tree, but yields really long rectangles!
	FVector2D Offset( FVector2D::ZeroVector );
	for( const auto& ChildNode : Node->Children )
	{
		ChildNode->Rect.Position = Node->Rect.Position + Offset;

		const float ChildFractionOfParent = ChildNode->Size / Node->Size;
		if( SplitDirection == ESplitDirection::Horizontal )
		{
			ChildNode->Rect.Size.X = Node->Rect.Size.X * ChildFractionOfParent;
			ChildNode->Rect.Size.Y = Node->Rect.Size.Y;
			Offset.X += ChildNode->Rect.Size.X;
		}
		else
		{
			ChildNode->Rect.Size.X = Node->Rect.Size.X;
			ChildNode->Rect.Size.Y = Node->Rect.Size.Y * ChildFractionOfParent;
			Offset.Y += ChildNode->Rect.Size.Y;
		}
	}
}


void FTreeMap::MakeSquarifiedTreeNode( const FTreeMapOptions& Options, const FTreeMapNodeRef& InNode )
{
	// NOTE: This algorithm is explained in the paper titled, "Squarified Treemaps", by Mark Bruls, Kees Huizing, and Jarke J.van Wijk

	// For squarification, we'll always choose the wider aspect direction at every split (ignoring incoming NextSplitDirection!)

	struct Local
	{
		/** Figure out the highest aspect ratio of all of the blocks, given the length of the rectangle that we want to place these blocks into */
		static float GetWorstAspectInRow( const TArray< FTreeMapNodePtr >& Nodes, const float SubRectShortestSide )
		{
			float MinSize = MAX_FLT;
			float MaxSize = 0.0f;
			float TotalSize = 0.0f;
			for( const auto& Node : Nodes )
			{
				MinSize = FMath::Min( MinSize, Node->Size );
				MaxSize = FMath::Max( MaxSize, Node->Size );
				TotalSize += Node->Size;
			}

			float TotalSizeSquared = TotalSize * TotalSize;
			float SubRectShortestSideSquared = SubRectShortestSide * SubRectShortestSide;

			float WorstAspect = FMath::Max( ( SubRectShortestSideSquared * MaxSize ) / TotalSizeSquared, TotalSizeSquared / ( SubRectShortestSideSquared * MinSize ) );
			return WorstAspect;
		}


		/** Incoming nodes should be sorted, largest to smallest */
		static TArray<FTreeMapNodePtr> BuildRowFromNodes( TArray<FTreeMapNodePtr>& Nodes, const float SubRectShortestSide )
		{
			TArray<FTreeMapNodePtr> Row;
					
			// Add the first child node to our row
			Row.Add( Nodes[ 0 ] );
			Nodes.RemoveAt( 0 );

			// If there are no more nodes to sort, then we're finished for now
			if( Nodes.Num() > 0 )
			{
				auto NewRow = Row;
				do
				{
					NewRow.Add( Nodes[ 0 ] );
					if( GetWorstAspectInRow( Row, SubRectShortestSide ) > GetWorstAspectInRow( NewRow, SubRectShortestSide ) )
					{
						Row = NewRow;

						// Claim the node from the original list
						Nodes.RemoveAt( 0 );
					}
					else
					{
						break;
					}
				}
				while( Nodes.Num() > 0 );
			}

			return Row;
		}


		/** Figures out which nodes will fit */
		static void PlaceNodes( TArray<FTreeMapNodePtr>& Row, FTreeMapRect& SubRect )
		{
			float TotalRowSize = 0.0f;
			for( const auto& Node : Row )
			{
				TotalRowSize += Node->Size;
			}

			const FVector2D SubRectMax = SubRect.Position + SubRect.Size;

			FTreeMapRect PlacementRect = SubRect;
			if( SubRect.Size.X < SubRect.Size.Y )
			{
				// Taller than wide
				float RowHeight = TotalRowSize / SubRect.Size.X;
				if( PlacementRect.Position.Y + RowHeight >= SubRectMax.Y )
				{
					RowHeight = SubRectMax.Y - PlacementRect.Position.Y;
				}

				for( int32 ColumnIndex = 0; ColumnIndex < Row.Num(); ++ColumnIndex )
				{
					auto& Node = Row[ ColumnIndex ];

					float Width = Node->Size / RowHeight;
					if( PlacementRect.Position.X + Width > SubRectMax.X || ( ColumnIndex + 1 ) == Row.Num() )
					{
						Width = SubRectMax.X - PlacementRect.Position.X;
					}
					Node->Rect.Position = PlacementRect.Position;
					Node->Rect.Size.X = Width;
					Node->Rect.Size.Y = RowHeight;

					PlacementRect.Position.X += Width;
				}

				const float NewY = SubRect.Position.Y + RowHeight;
				SubRect.Size.Y -= NewY - SubRect.Position.Y;
				SubRect.Position.Y = NewY;
			}
			else
			{
				// Wider than tall
				float RowWidth = TotalRowSize / SubRect.Size.Y;
				if( PlacementRect.Position.X + RowWidth >= SubRectMax.X )
				{
					RowWidth = SubRectMax.X - PlacementRect.Position.X;
				}

				for( int32 ColumnIndex = 0; ColumnIndex < Row.Num(); ++ColumnIndex )
				{
					auto& Node = Row[ ColumnIndex ];

					float Height = Node->Size / RowWidth;
					if( PlacementRect.Position.Y + Height > SubRectMax.Y || ( ColumnIndex + 1 ) == Row.Num() )
					{
						Height = SubRectMax.Y - PlacementRect.Position.Y;
					}
					Node->Rect.Position = PlacementRect.Position;
					Node->Rect.Size.X = RowWidth;
					Node->Rect.Size.Y = Height;

					PlacementRect.Position.Y += Height;
				}

				const float NewX = SubRect.Position.X + RowWidth;
				SubRect.Size.X -= NewX - SubRect.Position.X;
				SubRect.Position.X = NewX;
			}
		}
	};

	// Squarify it!
	auto ChildrenCopy = InNode->Children;
	FTreeMapRect SubRect = InNode->Rect;
	do
	{
		const auto SubRectShortestSide = FMath::Min( SubRect.Size.X, SubRect.Size.Y );
		auto Row = Local::BuildRowFromNodes( ChildrenCopy, SubRectShortestSide );
		Local::PlaceNodes( Row, SubRect );
	} 
	while( ChildrenCopy.Num() > 0 );
}


void FTreeMap::PartitionNodesRecursively( const FTreeMapOptions& Options, FTreeMap::ESplitDirection SplitDirection, const FTreeMapNodeRef& Node )
{
	// Store off our padded copy of the rectangle.  We'll actually do the padding later on.
	Node->PaddedRect = Node->Rect;

	if( !Node->IsLeafNode() )
	{
		if( Options.TreeMapType == ETreeMapType::Standard )
		{
			MakeStandardTreeNode( Options, SplitDirection, Node );
		}
		else if( Options.TreeMapType == ETreeMapType::Squarified )
		{
			MakeSquarifiedTreeNode( Options, Node );
		}

		// The default algorithm just alternates between horizontal and vertical.  The squarification algorithm ignores this.
		auto NextSplitDirection = ( SplitDirection == ESplitDirection::Horizontal ) ? ESplitDirection::Vertical : ESplitDirection::Horizontal;

		// Process children
		for( const auto& ChildNode : Node->Children )
		{
			PartitionNodesRecursively( Options, NextSplitDirection, ChildNode.ToSharedRef() );
		}
	}
}


void FTreeMap::PadNodesRecursively( const FTreeMapOptions& Options, const FTreeMapNodeRef& Node, const int32 TreeDepth )
{
	// Inset the container node to leave room for a border, if needed
	// Don't inset the root node
	const auto OriginalNodeRect = Node->Rect;

	// Choose a height for this node's font
	const uint16 MinAllowedFontSize = 8;		// @todo treemap custom: Don't hardcode and instead make this a customizable option?
	Node->NameFont = Options.NameFont; 
	Node->NameFont.Size = FMath::Max< int32 >( MinAllowedFontSize, Options.NameFont.Size - ( TreeDepth * Options.FontSizeChangeBasedOnDepth ) );
	Node->Name2Font = Options.Name2Font; 
	Node->Name2Font.Size = FMath::Max< int32 >( MinAllowedFontSize, Options.Name2Font.Size - ( TreeDepth * Options.FontSizeChangeBasedOnDepth ) );
	Node->CenterTextFont = Options.CenterTextFont; 
	Node->CenterTextFont.Size = FMath::Max< int32 >( MinAllowedFontSize, Options.CenterTextFont.Size - ( TreeDepth * Options.FontSizeChangeBasedOnDepth ) );

	if( Node != RootNode )
	{
		// Make sure we don't pad beyond our node's size
		const float ContainerOuterPadding = TreeDepth == 1 ? Options.TopLevelContainerOuterPadding : Options.NestedContainerOuterPadding;
		const FVector2D MaxPadding( Node->PaddedRect.Size * 0.5f );
		const FVector2D Padding( FMath::Min( ContainerOuterPadding, MaxPadding.X ), FMath::Min( ContainerOuterPadding, MaxPadding.Y ) );

		Node->PaddedRect.Position += Padding;
		Node->PaddedRect.Size -= Padding * 2.0f;
	}

	{
		// The 'child area' is the area within this node that we will fit all child nodes into
		auto ChildAreaRect = Node->PaddedRect;

		// Unless this is a top level node, make sure the node is big enough to bother reporting to our caller.  They may not want to visualize tiny nodes!
		Node->bIsVisible = ( ChildAreaRect.Size.X * ChildAreaRect.Size.Y >= Options.MinimumVisibleNodeSize );
		Node->bIsInteractive = ( TreeDepth <= 1 || ChildAreaRect.Size.X * ChildAreaRect.Size.Y >= Options.MinimumInteractiveNodeSize );

		if( Node->bIsInteractive && Node->bIsVisible )
		{
			// Figure out how much space we need for the title text
			const TSharedRef< FSlateFontMeasure >& FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const float MaxCharacterHeight = FontMeasureService->GetMaxCharacterHeight( Node->NameFont );		// @todo treemap perf: Cache this for various heights to reduce calls to FSlateFontMeasure
			const float ContainerTitleAreaHeight = MaxCharacterHeight;

			// Leave room for a title if we were asked to do that
			{
				const float Padding = FMath::Min( ChildAreaRect.Size.Y, ContainerTitleAreaHeight );
				ChildAreaRect.Position.Y += Padding;
				ChildAreaRect.Size.Y -= Padding;
			}

			// Apply inner padding before our child nodes, if needed
			{
				// Make sure we don't pad beyond our node's size
				const FVector2D MaxPadding( ChildAreaRect.Size * 0.5f );
				const FVector2D Padding( FMath::Min( Options.ContainerInnerPadding, MaxPadding.X ), FMath::Min( Options.ContainerInnerPadding, MaxPadding.Y ) );

				ChildAreaRect.Position += Padding;
				ChildAreaRect.Size -= Padding * 2.0f;
			}
		}

		// Offset and scale all of the child node rects to fit into the child area.  This is where some squashing might happen, 
		// and the sizes are no longer 1:1 with what they originally represented.  But for our purposes this is OK!  If you need
		// the sizes to be perfectly accurate, then disable all padding options.
		for( const auto& ChildNode : Node->Children )
		{
			ChildNode->PaddedRect.Position = ChildAreaRect.Position + ( ChildNode->PaddedRect.Position - OriginalNodeRect.Position ) / OriginalNodeRect.Size * ChildAreaRect.Size;
			ChildNode->PaddedRect.Size = ChildNode->PaddedRect.Size / OriginalNodeRect.Size * ChildAreaRect.Size;
		}
	}

	// Process children
	for( const auto& ChildNode : Node->Children )
	{
		PadNodesRecursively( Options, ChildNode.ToSharedRef(), TreeDepth + 1 );
	}
}


TSharedRef<ITreeMap> ITreeMap::CreateTreeMap( const FTreeMapOptions& Options, const FTreeMapNodeDataRef& RootNodeData )
{
	return MakeShareable( new FTreeMap( Options, RootNodeData ) );
}



FTreeMapNodeDataPtr ITreeMap::ParseOPMLToTreeMapData( const FString& OPMLFilePath, FString& OutErrorMessage )
{
	// Use the file name as the root node name
	const FString RootNodeName = FPaths::GetBaseFilename( OPMLFilePath );

	FXmlFile OPML;
	bool bLoadResult = OPML.LoadFile( OPMLFilePath );

	FTreeMapNodeDataPtr RootNodeData;
	if( bLoadResult && OPML.IsValid() )
	{
		// Get the working Xml Node
		const FXmlNode* XmlRoot = OPML.GetRootNode();
		if( XmlRoot != nullptr )
		{
			const auto& RootName = XmlRoot->GetTag();
			if( RootName.Equals( TEXT( "opml" ), ESearchCase::IgnoreCase ) )
			{
				for( const auto& OuterXmlNode : XmlRoot->GetChildrenNodes() )
				{
					const auto& OuterNodeName = OuterXmlNode->GetTag();
					if( OuterNodeName.Equals( TEXT( "body" ), ESearchCase::IgnoreCase ) )
					{
						struct Local
						{
							static void RecursivelyCreateNodes( const FTreeMapNodeDataRef& NodeData, const FXmlNode& XmlNode )
							{
								for( const auto& ChildXmlNode : XmlNode.GetChildrenNodes() )
								{
									const auto& NodeName = ChildXmlNode->GetTag();
									if( NodeName.Equals( TEXT( "outline" ), ESearchCase::IgnoreCase ) )
									{
										FTreeMapNodeDataRef ChildNodeData = MakeShareable( new FTreeMapNodeData() );
										ChildNodeData->Parent = &NodeData.Get();

										// All outline nodes MUST have a text attribute (required as part of OPML spec)
										const auto& OutlineText = ChildXmlNode->GetAttribute( TEXT( "text" ) );
										ChildNodeData->Name = OutlineText;
										NodeData->Children.Add( ChildNodeData );
										
										// Recurse into children
										RecursivelyCreateNodes( ChildNodeData, *ChildXmlNode );


										// Setup attributes of this node
										{
											const float DefaultLeafNodeSize = 1.0f;			// Leaf nodes must always have a non-zero size!
											const float DefaultContainerNodeSize = 0.0f;		// 0.0 for container nodes, means "compute my size using my children"
											ChildNodeData->Size = ChildNodeData->IsLeafNode() ? DefaultLeafNodeSize : DefaultContainerNodeSize;

											// Parse out any hash tags
											int32 HashTagCharIndex;
											while( ChildNodeData->Name.FindChar( '#', HashTagCharIndex ) )
											{
												// Parse hash tag string
												int32 HashTagLength = 1;
												while( ( ChildNodeData->Name.Len() > HashTagCharIndex + HashTagLength ) &&
													   !FChar::IsWhitespace( ChildNodeData->Name[ HashTagCharIndex + HashTagLength ] ) &&
													   ChildNodeData->Name[ HashTagCharIndex + HashTagLength ] != '#' )
												{
													++HashTagLength;
												}


												if( HashTagLength > 1 )
												{
													const FString HashTag = ChildNodeData->Name.Mid( HashTagCharIndex + 1, HashTagLength - 1 );

													ChildNodeData->HashTags.Add( HashTag );

													// Strip the hash tag ofg of the original string
													ChildNodeData->Name.MidInline( 0, HashTagCharIndex, EAllowShrinking::No );
													if( HashTagCharIndex + HashTagLength < ChildNodeData->Name.Len() )
													{
														ChildNodeData->Name += ChildNodeData->Name.Mid( HashTagCharIndex + HashTagLength );
													}
												}
											}

											// Clean up any leftover whitespace in the node name, after stripping out hash tags
											ChildNodeData->Name.TrimStartAndEndInline();
										}
									}
									else
									{
										// Node that we're not interested in
									}
								}
							}
						};

						RootNodeData = MakeShareable( new FTreeMapNodeData() );
						RootNodeData->Parent = NULL;
						RootNodeData->Name = RootNodeName;
						Local::RecursivelyCreateNodes( RootNodeData.ToSharedRef(), *OuterXmlNode );
					}
					else
					{
						// Top level node that we're not interested in
					}
				}

				if( !RootNodeData.IsValid() )
				{
					OutErrorMessage = TEXT( "Couldn't find a 'body' node in the XML document" );
				}

			}
			else
			{
				OutErrorMessage = TEXT( "File does not appear to be an OPML-formatted XML document" );
			}
		}
		else
		{
			OutErrorMessage = TEXT( "No root node found in XML document" );
		}
	}
	else
	{
		// Couldn't load file
		OutErrorMessage = OPML.GetLastError();
	}

	return RootNodeData;
}


TArray<FTreeMapNodeVisualInfo> FTreeMap::GetVisuals()
{
	TArray<FTreeMapNodeVisualInfo> Visuals;

	struct Local
	{
		static void RecursivelyGatherVisuals( TArray<FTreeMapNodeVisualInfo>& VisualsList, const FTreeMapNodeRef& Node )
		{
			if (!Node->bIsVisible)
			{
				return;
			}

			// Add a visual for the node that was passed in.  We'll recurse down into children afterwards.
			FTreeMapNodeVisualInfo Visual;
			Visual.NodeData = Node->Data.Get();
			Visual.Position = Node->PaddedRect.Position;
			Visual.Size = Node->PaddedRect.Size;
			Visual.Color = Node->Data->Color;
			Visual.NameFont = Node->NameFont;
			Visual.Name2Font = Node->Name2Font;
			Visual.CenterTextFont = Node->CenterTextFont;
			Visual.bIsInteractive = Node->bIsInteractive;

			// If the node is non-interactive, then ghost it
			if( !Visual.bIsInteractive )
			{
				Visual.Color.A *= 0.25f;
			}

			VisualsList.Add( Visual );


			// Process children
			for( auto ChildNodeIndex = 0; ChildNodeIndex < Node->Children.Num(); ++ChildNodeIndex )
			{
				const auto& ChildNode = Node->Children[ ChildNodeIndex ];

				// Make up a distinct color for all of the root's top level nodes
				RecursivelyGatherVisuals( VisualsList, ChildNode.ToSharedRef() );
			}
		}
	};

	Local::RecursivelyGatherVisuals( Visuals, RootNode.ToSharedRef() );

	return Visuals;
}
