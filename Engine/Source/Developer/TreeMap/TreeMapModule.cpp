// Copyright Epic Games, Inc. All Rights Reserved.

#include "TreeMapModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "TreeMapStyle.h"

/**
 * Implements the TreeMap module.
 */
class FTreeMapModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override
	{
		FTreeMapStyle::Initialize();
	}

	virtual void ShutdownModule() override
	{
		FTreeMapStyle::Shutdown();
	}
};


IMPLEMENT_MODULE(FTreeMapModule, TreeMap);
