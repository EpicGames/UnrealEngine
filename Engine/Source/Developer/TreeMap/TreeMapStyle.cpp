// Copyright Epic Games, Inc. All Rights Reserved.

#include "TreeMapStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( FPaths::EngineContentDir() / "Editor/Slate"/ RelativePath + TEXT(".png"), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( FPaths::EngineContentDir() / "Editor/Slate"/ RelativePath + TEXT(".png"), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( FPaths::EngineContentDir() / "Editor/Slate"/ RelativePath + TEXT(".png"), __VA_ARGS__ )

TSharedPtr< FSlateStyleSet > FTreeMapStyle::StyleInstance = nullptr;

void FTreeMapStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FTreeMapStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FTreeMapStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("TreeMapStyle"));
	return StyleSetName;
}

TSharedRef< FSlateStyleSet > FTreeMapStyle::Create()
{
	TSharedRef<FSlateStyleSet> StyleRef = MakeShareable(new FSlateStyleSet(FTreeMapStyle::GetStyleSetName()));
	StyleRef->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleRef->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FSlateStyleSet& Style = StyleRef.Get();

	// Tree Map
	Style.Set( "TreeMap.Background", new BOX_BRUSH( "TreeMap/TreeMap_Background", FMargin( 5.f / 12.f ) ) );
	Style.Set( "TreeMap.NodeBackground", new BOX_BRUSH( "TreeMap/TreeMap_NodeBackground", FMargin( 5.f / 12.f ) ) );
	Style.Set( "TreeMap.HoveredNodeBackground", new BOX_BRUSH( "TreeMap/TreeMap_NodeHoveredBackground", FMargin(5.f/12.f) ) );
	Style.Set( "TreeMap.BorderPadding", FVector2D(1,0) );

	return StyleRef;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH

const ISlateStyle& FTreeMapStyle::Get()
{
	return *StyleInstance;
}
